{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Apple ARKit",
	"Description": "Support for Apple's ARKit augmented reality system",
	"Category": "Augmented Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"EnabledByDefault": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"SupportedTargetPlatforms" : [
		"IOS", "Win64", "Mac","Linux", "Android"
	],
	"Modules": [
		{
			"Name": "AppleARKit",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"ProgramAllowList": ["LiveLinkHub"],
			"PlatformAllowList" : [ "Mac", "IOS", "Win64","Linux", "Android" ]
		},
		{
			"Name": "AppleARKitPoseTrackingLiveLink",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"ProgramAllowList": ["LiveLinkHub"],
			"PlatformAllowList" : [ "Mac", "IOS", "Win64" ,"Linux", "Android" ]

		}
	],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "AppleImageUtils",
			"Enabled": true
		},
		{
			"Name": "ARUtilities",
			"Enabled": true
		}
	]
}
