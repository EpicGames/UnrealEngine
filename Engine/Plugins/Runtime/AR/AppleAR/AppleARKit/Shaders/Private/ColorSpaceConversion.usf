// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

RWTexture2D<float4> OutputTexture;
float2 OutputTextureSize;

Texture2D InputTextureY;
float2 InputTextureYSize;

Texture2D InputTextureCbCr;
float2 InputTextureCbCrSize;

/**
 * The device's current orientation
 * 0 - Portrait
 * 1 - PortraitUpsideDown
 * 2 - LandscapeLeft
 * 3 - LandscapeRight
 */
int DeviceOrientation;

float2 GetInputUV(float2 OutputUV)
{
	if (DeviceOrientation == 0)
	{
		// Portrait
		return float2(OutputUV.y, 1.0 - OutputUV.x);
	}
	else if (DeviceOrientation == 1)
	{
		// PortraitUpsideDown
		return float2(1.0 - OutputUV.y, OutputUV.x);
	}
	else if (DeviceOrientation == 2)
	{
		// LandscapeLeft
		return float2(1.0 - OutputUV.x, 1.0 - OutputUV.y);
	}
	else
	{
		// LandscapeRight
		return OutputUV;
	}
}

[numthreads(THREADGROUPSIZE, THREADGROUPSIZE, 1)]
void YCbCrToLinearRGB(uint3 ThreadId : SV_DispatchThreadID)
{
	float2 iResolution = float2(OutputTextureSize.x, OutputTextureSize.y);
	float2 UV = (ThreadId.xy / iResolution.xy);
	UV = GetInputUV(UV);
	
	// See http://www.mir.com/DMG/ycbcr.html
	float4 ColorY = InputTextureY.Load(int3(UV * InputTextureYSize, 0));
	float4 ColorCbCr = InputTextureCbCr.Load(int3(UV * InputTextureCbCrSize, 0)) - 0.5;
	float4 InputColor = float4(ColorY.x, ColorCbCr.xy, 1);
	float4x4 YCbCrConvert =
	{
	  1.0000, 0.0000, 1.4020, 0,
	  1.0000, -0.3441, -0.7141, 0,
	  1.0000, 1.7720, 0.0000, 0,
	  0.0000, 0.0000, 0.0000, 1.0000
	};
	float4 sRGB = mul(YCbCrConvert, InputColor);
	// https://entropymine.com/imageworsener/srgbformula/
	// Note that this is a simplified conversion, ignoring the very dark pixels (<= 0.04045)
	// which should be converted as L = S/12.92
	float4 LinearColor = pow(sRGB * (1.0 / 1.055) + 0.0521327, 2.4);
	OutputTexture[ThreadId.xy] = LinearColor;
}
