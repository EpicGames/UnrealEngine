// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AppleARKit : ModuleRules
{
	public AppleARKit(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateIncludePaths.AddRange(new string[]
		{
		});
			
		
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Engine",
			"MRMesh",
			"EyeTracker",			
		});
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"Slate",
			"SlateCore",
			"RHI",
			"Renderer",
			"RenderCore",
			"HeadMountedDisplay",
			"XRBase",
			"AugmentedReality",
			"AppleImageUtils",
			"Projects",
			"ARUtilities",
		});
            
		DynamicallyLoadedModuleNames.AddRange(new string[]
		{
		});
		
		
		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
    		PrivateDependencyModuleNames.Add("IOSRuntimeSettings");
			
			PublicFrameworks.AddRange(new string[]
			{
				"ARKit",
				"MetalPerformanceShaders",
				"CoreLocation",
			});
			
			PrivateDependencyModuleNames.AddRange(new string[] {
        	    "MetalRHI",
        	});
        	
			PublicWeakFrameworks.Add("Metal");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
		
			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
            AdditionalPropertiesForReceipt.Add("IOSPlugin", Path.Combine(PluginPath, "AppleARKit_IOS_UPL.xml"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.AddRange(new string[]
			{
				"MetalPerformanceShaders",
			});
			
			PrivateDependencyModuleNames.AddRange(new string[] {
        	    "MetalRHI",
        	});
			
			PublicWeakFrameworks.Add("Metal");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
		}
	}
}
