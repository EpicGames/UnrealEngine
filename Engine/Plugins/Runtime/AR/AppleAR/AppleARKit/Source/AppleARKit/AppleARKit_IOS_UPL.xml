<?xml version="1.0" encoding="utf-8"?>
<root>

    <init>
        <log text="AppleARKit adding arkit requirement and camera permission to plist..."/>
    </init>

    <iosPListUpdates>
        <addElements tag="dict" once="true">
            <key>NSCameraUsageDescription</key>
            <string>The camera is used for augmenting reality.</string>
        </addElements>

		<setBoolFromProperty result="bRequireARKitSupport" ini="Engine" section="/Script/AppleARKit.AppleARKitSettings" property="bRequireARKitSupport" default="true"/>
		<if condition="bRequireARKitSupport">
			<true>
                <loopElements tag="dict">
                    <setBool result="bReqKey" value="false"/>

                    <loopElements tag="$">
                        <setStringFromTag result="TagName" tag="$"/>

                        <setBoolIsEqual result="bIsKey" arg1="$S(TagName)" arg2="key"/>
                        <if condition="bIsKey">
                            <true>
                                <setStringFromTagText result="TagValue" tag="$"/>
                                <setBoolIsEqual result="bReqKey" arg1="$S(TagValue)" arg2="UIRequiredDeviceCapabilities"/>
                            </true>
                        </if>

                        <setBoolIsEqual result="bIsArray" arg1="$S(TagName)" arg2="array"/>
                        <if condition="bIsArray">
                            <true>
                                <if condition="bReqKey">
                                    <true>
                                        <setElement result="NewEntry" xml="&lt;string&gt;arkit&lt;/string&gt;"/>
                                        <addElement tag="$" name="NewEntry"/>
                                    </true>
                                </if>
                            </true>
                        </if>

                    </loopElements>
                </loopElements>
			</true>
		</if>
    </iosPListUpdates>

</root>
