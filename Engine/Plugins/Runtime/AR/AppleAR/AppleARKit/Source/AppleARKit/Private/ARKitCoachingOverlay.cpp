// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARKitCoachingOverlay.h"
#include "AppleARKitSystem.h"
#include "AppleARKitModule.h"

#if SUPPORTS_ARKIT_3_0
#include "IOS/IOSView.h"
#include "IOS/IOSAppDelegate.h"

@implementation FARKitCoachingOverlay
{
	ARCoachingOverlayView* CoachingOverlay;
	FAppleARKitSystem* AppleARKitSystem;
}

- (id)initWithAppleARKitSystem:(FAppleARKitSystem*)InAppleARKitSystem
{
	self = [super init];
	if (self)
	{
		AppleARKitSystem = InAppleARKitSystem;
		if (FAppleARKitAvailability::SupportsARKit30())
		{
			dispatch_sync(dispatch_get_main_queue(), ^{
				// init must happen in the main thread
				CoachingOverlay = [[ARCoachingOverlayView alloc] init];
				CoachingOverlay.translatesAutoresizingMaskIntoConstraints = false;
			});
			
			CoachingOverlay.delegate = self;
			CoachingOverlay.activatesAutomatically = true;
			CoachingOverlay.goal = ARCoachingGoalHorizontalPlane;
		}
	}
	return self;
}

- (void)dealloc
{
	if (FAppleARKitAvailability::SupportsARKit30() && CoachingOverlay)
	{
		// use a copy of the overlay object as self is being destroyed
		// thus cannot be used in the async block below
		auto CoachingOverlayCopy = CoachingOverlay;
		dispatch_async(dispatch_get_main_queue(), ^{
			// removeFromSuperview must be called in the main thread
			[CoachingOverlayCopy removeFromSuperview];
			[CoachingOverlayCopy release];
		});
		CoachingOverlay = nullptr;
	}
	[super dealloc];
}

- (void)setARSession:(ARSession*)InARSession
{
	if (FAppleARKitAvailability::SupportsARKit30() && CoachingOverlay)
	{
		CoachingOverlay.session = InARSession;
	}
}

- (void)addToRootView
{
	if (FAppleARKitAvailability::SupportsARKit30() && CoachingOverlay)
	{
		if (auto RootView = [IOSAppDelegate GetDelegate].RootView)
		{
			dispatch_async(dispatch_get_main_queue(), ^{
				[RootView addSubview: CoachingOverlay];
				// set the constraints so that the overlay fills the screen
				[CoachingOverlay.centerXAnchor constraintEqualToAnchor: RootView.centerXAnchor].active = YES;
				[CoachingOverlay.centerYAnchor constraintEqualToAnchor: RootView.centerYAnchor].active = YES;
				[CoachingOverlay.widthAnchor constraintEqualToAnchor: RootView.widthAnchor].active = YES;
				[CoachingOverlay.heightAnchor constraintEqualToAnchor: RootView.heightAnchor].active = YES;
			});
		}
	}
}

- (void)coachingOverlayViewWillActivate:(ARCoachingOverlayView *)coachingOverlayView
{
	// TODO: notify the user about this event
	UE_LOGF(LogAppleARKit, Log, "FARKitCoachingOverlay: coachingOverlayViewWillActivate");
}

- (void)coachingOverlayViewDidDeactivate:(ARCoachingOverlayView *)coachingOverlayView
{
	// TODO: notify the user about this event
	UE_LOGF(LogAppleARKit, Log, "FARKitCoachingOverlay: coachingOverlayViewDidDeactivate");
}

- (void)coachingOverlayViewDidRequestSessionReset:(ARCoachingOverlayView *)coachingOverlayView
{
	// TODO: notify the user about this event
	UE_LOGF(LogAppleARKit, Log, "FARKitCoachingOverlay: coachingOverlayViewDidRequestSessionReset");
}
@end

#endif
