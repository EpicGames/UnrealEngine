// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AppleARKitAvailability.h"

#if SUPPORTS_ARKIT_3_0
#include <ARKit/ARKit.h>

@interface FARKitCoachingOverlay : NSObject<ARCoachingOverlayViewDelegate>
{
}

- (id)initWithAppleARKitSystem:(class FAppleARKitSystem*)InAppleARKitSystem;

- (void)setARSession:(ARSession*)InARSession;

- (void)addToRootView;
@end

#endif
