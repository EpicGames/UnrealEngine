// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARKitGeoTrackingSupport.h"
#include "AppleARKitAvailability.h"
#include "Async/Async.h"
#include "ARSupportInterface.h"
#include "Features/IModularFeatures.h"
#include "AppleARKitModule.h"
#include "AppleARKitConversion.h"
#include "HAL/IConsoleManager.h"

#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	#define GIgnoreGeoTrackingState 0
#else
	static int32 GIgnoreGeoTrackingState = 0;
	static FAutoConsoleVariableRef CVarIgnoreGeoTrackingState(
		TEXT("arkit.Debug.IgnoreGeoTrackingState"),
		GIgnoreGeoTrackingState,
		TEXT("This is a debug variable that skips the Geo tracking state check before calling certain APIs.")
		);
#endif

#if PLATFORM_APPLE
static FString GetErrorString(NSError* Error)
{
	FString ErrorString;
	if (Error)
	{
		ErrorString = Error.localizedDescription;
	}
	return MoveTemp(ErrorString);
}
#endif

#if SUPPORTS_ARKIT_4_0
EARGeoTrackingState FAppleARKitConversion::ToGeoTrackingState(ARGeoTrackingState InState)
{
	static const TMap<int32, EARGeoTrackingState> Mapping =
	{
		{ (int32)ARGeoTrackingStateInitializing, 	EARGeoTrackingState::Initializing },
		{ (int32)ARGeoTrackingStateLocalized, 		EARGeoTrackingState::Localized },
		{ (int32)ARGeoTrackingStateLocalizing, 		EARGeoTrackingState::Localizing },
		{ (int32)ARGeoTrackingStateNotAvailable, 	EARGeoTrackingState::NotAvailable },
	};
	
	if (auto Record = Mapping.Find((int32)InState))
	{
		return *Record;
	}
	
	return EARGeoTrackingState::NotAvailable;
}

EARGeoTrackingStateReason FAppleARKitConversion::ToGeoTrackingStateReason(ARGeoTrackingStateReason InReason)
{
	static const TMap<int32, EARGeoTrackingStateReason> Mapping =
	{
		{ (int32)ARGeoTrackingStateReasonNone, 							EARGeoTrackingStateReason::None },
		{ (int32)ARGeoTrackingStateReasonNotAvailableAtLocation, 		EARGeoTrackingStateReason::NotAvailableAtLocation },
		{ (int32)ARGeoTrackingStateReasonNeedLocationPermissions, 		EARGeoTrackingStateReason::NeedLocationPermissions },
		{ (int32)ARGeoTrackingStateReasonDevicePointedTooLow, 			EARGeoTrackingStateReason::DevicePointedTooLow },
		{ (int32)ARGeoTrackingStateReasonWorldTrackingUnstable, 		EARGeoTrackingStateReason::WorldTrackingUnstable },
		{ (int32)ARGeoTrackingStateReasonWaitingForLocation, 			EARGeoTrackingStateReason::WaitingForLocation },
		{ (int32)ARGeoTrackingStateReasonGeoDataNotLoaded, 				EARGeoTrackingStateReason::GeoDataNotLoaded },
		{ (int32)ARGeoTrackingStateReasonVisualLocalizationFailed, 		EARGeoTrackingStateReason::VisualLocalizationFailed },
		{ (int32)ARGeoTrackingStateReasonWaitingForAvailabilityCheck, 	EARGeoTrackingStateReason::WaitingForAvailabilityCheck },
	};
	
	if (auto Record = Mapping.Find((int32)InReason))
	{
		return *Record;
	}
	
	return EARGeoTrackingStateReason::NotAvailableAtLocation;
}

EARGeoTrackingAccuracy FAppleARKitConversion::ToGeoTrackingAccuracy(ARGeoTrackingAccuracy InAccuracy)
{
	static const TMap<int32, EARGeoTrackingAccuracy> Mapping =
	{
		{ (int32)ARGeoTrackingAccuracyUndetermined, EARGeoTrackingAccuracy::Undetermined },
		{ (int32)ARGeoTrackingAccuracyLow, 			EARGeoTrackingAccuracy::Low },
		{ (int32)ARGeoTrackingAccuracyMedium, 		EARGeoTrackingAccuracy::Medium },
		{ (int32)ARGeoTrackingAccuracyHigh, 		EARGeoTrackingAccuracy::High },
	};
	
	if (auto Record = Mapping.Find((int32)InAccuracy))
	{
		return *Record;
	}
	
	return EARGeoTrackingAccuracy::Undetermined;
}

EARAltitudeSource FAppleARKitConversion::ToAltitudeSource(ARAltitudeSource InSource)
{
	static const TMap<int32, EARAltitudeSource> Mapping =
	{
		{ (int32)ARAltitudeSourcePrecise, 		EARAltitudeSource::Precise },
		{ (int32)ARAltitudeSourceCoarse, 		EARAltitudeSource::Coarse },
		{ (int32)ARAltitudeSourceUserDefined, 	EARAltitudeSource::UserDefined },
		{ (int32)ARAltitudeSourceUnknown, 		EARAltitudeSource::Unknown },
	};
	
	if (auto Record = Mapping.Find((int32)InSource))
	{
		return *Record;
	}
	
	return EARAltitudeSource::Unknown;
}
#endif


static FARSupportInterface* GetARSupportInterface()
{
	if (auto Implementation = IModularFeatures::Get().GetModularFeatureImplementation(FARSupportInterface::GetModularFeatureName(), 0))
	{
		return static_cast<FARSupportInterface*>(Implementation);
	}
	return nullptr;
}

#if SUPPORTS_ARKIT_1_0
static ARFrame* GetCurrentFrame()
{
	if (auto ARSupport = GetARSupportInterface())
	{
		return static_cast<ARFrame*>(ARSupport->GetGameThreadARFrameRawPointer());
	}
	return nullptr;
}


static ARSession* GetCurrentSession()
{
	if (auto ARSupport = GetARSupportInterface())
	{
		return static_cast<ARSession*>(ARSupport->GetARSessionRawPointer());
	}
	return nullptr;
}
#endif

TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> UARKitGeoTrackingSupport::CheckGeoTrackingAvailability(FString& OutError)
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (ARGeoTrackingConfiguration.isSupported)
		{
			auto Task = MakeShared<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe>();
			[ARGeoTrackingConfiguration checkAvailabilityWithCompletionHandler: ^(BOOL isAvailable, NSError* Error)
			{
				const auto ErrorString = GetErrorString(Error);
				AsyncTask(ENamedThreads::GameThread, [isAvailable, ErrorString, Task]
				{
					if (ErrorString.Len())
					{
						Task->FinishWithError(ErrorString);
					}
					else
					{
						Task->FinishWithAvailability(isAvailable);
					}
				});
			}];
			
			return Task;
		}
	}
#endif
	
	OutError = TEXT("Geo tracking is not supported");
	return nullptr;
}

TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> UARKitGeoTrackingSupport::CheckGeoTrackingAvailability(float Longitude, float Latitude, FString& OutError)
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (ARGeoTrackingConfiguration.isSupported)
		{
			auto Task = MakeShared<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe>();
			[ARGeoTrackingConfiguration checkAvailabilityAtCoordinate: CLLocationCoordinate2DMake(Latitude, Longitude)
													completionHandler: ^(BOOL isAvailable, NSError* Error)
			{
				const auto ErrorString = GetErrorString(Error);
				AsyncTask(ENamedThreads::GameThread, [isAvailable, ErrorString, Task]
				{
					if (ErrorString.Len())
					{
						Task->FinishWithError(ErrorString);
					}
					else
					{
						Task->FinishWithAvailability(isAvailable);
					}
				});
			}];
			
			return Task;
		}
	}
#endif
	
	OutError = TEXT("Geo tracking is not supported");
	return nullptr;
}

EARGeoTrackingState UARKitGeoTrackingSupport::GetGeoTrackingState() const
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (auto Frame = GetCurrentFrame())
		{
			if (Frame.geoTrackingStatus)
			{
				return FAppleARKitConversion::ToGeoTrackingState(Frame.geoTrackingStatus.state);
			}
		}
	}
#endif
	return EARGeoTrackingState::NotAvailable;
}

EARGeoTrackingStateReason UARKitGeoTrackingSupport::GetGeoTrackingStateReason() const
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (auto Frame = GetCurrentFrame())
		{
			if (Frame.geoTrackingStatus)
			{
				return FAppleARKitConversion::ToGeoTrackingStateReason(Frame.geoTrackingStatus.stateReason);
			}
		}
	}
#endif
	return EARGeoTrackingStateReason::NotAvailableAtLocation;
}

EARGeoTrackingAccuracy UARKitGeoTrackingSupport::GetGeoTrackingAccuracy() const
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (auto Frame = GetCurrentFrame())
		{
			if (Frame.geoTrackingStatus)
			{
				return FAppleARKitConversion::ToGeoTrackingAccuracy(Frame.geoTrackingStatus.accuracy);
			}
		}
	}
#endif
	return EARGeoTrackingAccuracy::Undetermined;
}

bool UARKitGeoTrackingSupport::AddGeoAnchorAtLocation(float Longitude, float Latitude, FString OptionalAnchorName)
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (GetGeoTrackingState() == EARGeoTrackingState::Localized || GIgnoreGeoTrackingState)
		{
			if (auto Session = GetCurrentSession())
			{
				ARGeoAnchor* Anchor = nullptr;
				if (OptionalAnchorName.Len())
				{
					Anchor = [[ARGeoAnchor alloc] initWithName: OptionalAnchorName.GetNSString()
													coordinate: CLLocationCoordinate2DMake(Latitude, Longitude)];
				}
				else
				{
					Anchor = [[ARGeoAnchor alloc] initWithCoordinate: CLLocationCoordinate2DMake(Latitude, Longitude)];
				}
				[Session addAnchor: Anchor];
				return true;
			}
		}
		else
		{
			UE_LOGF(LogAppleARKit, Warning, "Geo Anchor can only be added when the Geo tracking state is Localized!")
		}
	}
#endif
	return false;
}

bool UARKitGeoTrackingSupport::AddGeoAnchorAtLocationWithAltitude(float Longitude, float Latitude, float AltitudeMeters, FString OptionalAnchorName)
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (GetGeoTrackingState() == EARGeoTrackingState::Localized || GIgnoreGeoTrackingState)
		{
			if (auto Session = GetCurrentSession())
			{
				ARGeoAnchor* Anchor = nullptr;
				if (OptionalAnchorName.Len())
				{
					Anchor = [[ARGeoAnchor alloc] initWithName: OptionalAnchorName.GetNSString()
													coordinate: CLLocationCoordinate2DMake(Latitude, Longitude)
													  altitude: AltitudeMeters];
				}
				else
				{
					Anchor = [[ARGeoAnchor alloc] initWithCoordinate: CLLocationCoordinate2DMake(Latitude, Longitude)
															altitude: AltitudeMeters];
				}
				[Session addAnchor: Anchor];
				return true;
			}
		}
		else
		{
			UE_LOGF(LogAppleARKit, Warning, "Geo Anchor can only be added when the Geo tracking state is Localized!")
		}
	}
#endif
	return false;
}

TSharedPtr<FGetGeoLocationAsyncTask, ESPMode::ThreadSafe> UARKitGeoTrackingSupport::GetGeoLocationAtWorldPosition(const FVector& WorldPosition, FString& OutError)
{
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (ARGeoTrackingConfiguration.isSupported)
		{
			if (GetGeoTrackingState() == EARGeoTrackingState::Localized || GIgnoreGeoTrackingState)
			{
				auto ARSupport = GetARSupportInterface();
				auto Session = GetCurrentSession();
				
				if (ARSupport && Session)
				{
					const auto& AlignmentTransform = ARSupport->GetAlignmentTransform();
					auto TrackingToWorldTransform = FTransform::Identity;
					if (auto TrackingSystem = ARSupport->GetXRTrackingSystem())
					{
						TrackingToWorldTransform = TrackingSystem->GetTrackingToWorldTransform();
					}
					const auto LocalToWorldTransform = AlignmentTransform * TrackingToWorldTransform;
					const auto LocalPosition = LocalToWorldTransform.InverseTransformPosition(WorldPosition);
					const auto ARKitPosition = FAppleARKitConversion::ToARKitVector(LocalPosition);
					
					auto Task = MakeShared<FGetGeoLocationAsyncTask, ESPMode::ThreadSafe>();
					
					[Session getGeoLocationForPoint: ARKitPosition
								  completionHandler: ^(CLLocationCoordinate2D Coordinate, CLLocationDistance Altitude, NSError* Error)
					{
						const auto ErrorString = GetErrorString(Error);
						AsyncTask(ENamedThreads::GameThread, [Coordinate, Altitude, ErrorString, Task]
						{
							if (ErrorString.Len())
							{
								Task->FinishWithError(ErrorString);
							}
							else
							{
								Task->FinishWithGeoLocation(Coordinate.longitude, Coordinate.latitude, Altitude);
							}
						});
					}];
					
					return Task;
				}
				else
				{
					OutError = TEXT("Failed to get AR session");
				}
			}
			else
			{
				OutError = TEXT("Geo tracking state is not localized");
			}
		}
	}
#endif
	
	if (!OutError.Len())
	{
		OutError = TEXT("Geo tracking is not supported");
	}
	return nullptr;
}
