// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARGeoTrackingSupport.h"

#include "ARKitGeoTrackingSupport.generated.h"


UCLASS()
class APPLEARKIT_API UARKitGeoTrackingSupport : public UARGeoTrackingSupport
{
	GENERATED_BODY()
	
public:
	TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> CheckGeoTrackingAvailability(FString& OutError) override;
	TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> CheckGeoTrackingAvailability(float Longitude, float Latitude, FString& OutError) override;
	TSharedPtr<FGetGeoLocationAsyncTask, ESPMode::ThreadSafe> GetGeoLocationAtWorldPosition(const FVector& WorldPosition, FString& OutError) override;
	
	EARGeoTrackingState GetGeoTrackingState() const override;
	EARGeoTrackingStateReason GetGeoTrackingStateReason() const override;
	EARGeoTrackingAccuracy GetGeoTrackingAccuracy() const override;
	
	bool AddGeoAnchorAtLocation(float Longitude, float Latitude, FString OptionalAnchorName) override;
	bool AddGeoAnchorAtLocationWithAltitude(float Longitude, float Latitude, float AltitudeMeters, FString OptionalAnchorName) override;
};
