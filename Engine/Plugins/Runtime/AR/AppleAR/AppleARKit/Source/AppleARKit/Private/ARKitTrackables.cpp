// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARKitTrackables.h"
#include "AppleARKitModule.h"
#include "AppleARKitSystem.h"
#include "AppleARKitMeshData.h"
#include "AppleARKitConversion.h"


bool UARKitMeshGeometry::GetObjectClassificationAtLocation(const FVector& InWorldLocation, EARObjectClassification& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff)
{
#if SUPPORTS_ARKIT_3_5
	if (auto ARKitSystem = FAppleARKitModule::GetARKitSystem())
	{
		FGuid MyGuid;
		if (ARKitSystem->GetGuidForGeometry(this, MyGuid))
		{
			if (auto MeshData = FARKitMeshData::GetMeshData(MyGuid))
			{
				uint8 Classification = 0;
				if (MeshData->GetClassificationAtLocation(InWorldLocation, GetLocalToWorldTransform(), Classification, OutClassificationLocation, MaxLocationDiff))
				{
					OutClassification = FAppleARKitConversion::ToEARObjectClassification((ARMeshClassification)Classification);
					return true;
				}
			}
		}
	}
#endif
	
	return false;
}
