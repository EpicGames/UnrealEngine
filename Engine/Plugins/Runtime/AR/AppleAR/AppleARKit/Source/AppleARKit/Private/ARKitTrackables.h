// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ARTrackable.h"

#include "ARKitTrackables.generated.h"

UCLASS(NotBlueprintType)
class UARKitMeshGeometry : public UARMeshGeometry
{
	GENERATED_BODY()
	
public:
	bool GetObjectClassificationAtLocation(const FVector& InWorldLocation, EARObjectClassification& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff = 10.f) override;
};
