// Copyright Epic Games, Inc. All Rights Reserved.

// AppleARKit
#include "AppleARKitCamera.h"
#include "AppleARKitModule.h"
#include "AppleARKitConversion.h"
	
// UE
#include "CoreGlobals.h"
#include "Engine/EngineTypes.h"
#include "Kismet/GameplayStatics.h"
#include "Rendering/RenderingCommon.h"
#include "SceneView.h"

#if SUPPORTS_ARKIT_1_0

FAppleARKitCamera::FAppleARKitCamera( ARCamera* InARCamera )
{
	// Sanity check
	check( InARCamera );

	// Copy / convert timestamp & camera intrinsics
	FocalLength.X = InARCamera.intrinsics.columns[0][0];
    FocalLength.Y = InARCamera.intrinsics.columns[1][1];
    PrincipalPoint.X = InARCamera.intrinsics.columns[2][0];
    PrincipalPoint.Y = InARCamera.intrinsics.columns[2][1];
    ImageResolution.X = InARCamera.imageResolution.width;
    ImageResolution.Y = InARCamera.imageResolution.height;

	// Tracking quality
	switch(InARCamera.trackingState)
	{
		default:
		case ARTrackingStateNotAvailable:
			TrackingQuality = EARTrackingQuality::NotTracking;
			break;
			
		case ARTrackingStateLimited:
			TrackingQuality = EARTrackingQuality::OrientationOnly;
			break;
		
		case ARTrackingStateNormal:
			TrackingQuality = EARTrackingQuality::OrientationAndPosition;
			break;
	}
	
	// tracking quality reason
	switch (InARCamera.trackingStateReason)
	{
		default:
		case ARTrackingStateReasonNone:
			TrackingQualityReason = EARTrackingQualityReason::None;
			break;

		case ARTrackingStateReasonInitializing:
			TrackingQualityReason = EARTrackingQualityReason::Initializing;
			break;

		case ARTrackingStateReasonRelocalizing:
			TrackingQualityReason = EARTrackingQualityReason::Relocalizing;
			break;

		case ARTrackingStateReasonExcessiveMotion:
			TrackingQualityReason = EARTrackingQualityReason::ExcessiveMotion;
			break;

		case ARTrackingStateReasonInsufficientFeatures:
			TrackingQualityReason = EARTrackingQualityReason::InsufficientFeatures;
			break;
	}
	
	
    // Copy / convert camera transform
    Transform = FAppleARKitConversion::ToFTransform( InARCamera.transform );
    Translation = Transform.GetTranslation();
	Orientation = Transform.GetRotation();
	
}

#endif

float FAppleARKitCamera::GetAspectRatio() const
{
	return ImageResolution.X / ImageResolution.Y;
}

float FAppleARKitCamera::GetHorizontalFieldOfView() const
{
	return (FocalLength.X > 0.0f)
		? FMath::RadiansToDegrees( 2.0f * FMath::Atan( ( ImageResolution.X / 2.0f ) / FocalLength.X ) )
		: 0.0f;
}

float FAppleARKitCamera::GetVerticalFieldOfView() const
{
	return (FocalLength.Y > 0.0f)
		? FMath::RadiansToDegrees( 2.0f * FMath::Atan( ( ImageResolution.Y / 2.0f ) / FocalLength.Y ) )
		: 0.0f;
}

float FAppleARKitCamera::GetHorizontalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode ) const
{
	// Use the global viewport size as the screen size
	FVector2D ViewportSize;
	GEngine->GameViewport->GetViewportSize( ViewportSize );
	
	return GetHorizontalFieldOfViewForScreen( BackgroundFitMode, ViewportSize.X, ViewportSize.Y );
}

float FAppleARKitCamera::GetVerticalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode ) const
{
	// Use the global viewport size as the screen size
	FVector2D ViewportSize;
	GEngine->GameViewport->GetViewportSize( ViewportSize );
	
	return GetVerticalFieldOfViewForScreen( BackgroundFitMode, ViewportSize.X, ViewportSize.Y );
}

float FAppleARKitCamera::GetHorizontalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode, float ScreenWidth, float ScreenHeight ) const
{
	// Sanity check?
	if ( FocalLength.X <= 0.0f )
	{
		return 0.0f;
	}

	// Are they the same aspect ratio anyway?
	float ScreenAspectRatio = ScreenWidth / ScreenHeight;
	float CameraAspectRatio = GetAspectRatio();
	if (FMath::IsNearlyEqual(ScreenAspectRatio, CameraAspectRatio))
	{
		return GetHorizontalFieldOfView();
	}

	// Not matching, figure out FOV for fit mode
	switch ( BackgroundFitMode )
	{
		/** The background image will be letterboxed to fit the screen */
		case EAppleARKitBackgroundFitMode::Fit:
		{
			// @todo implement Fir FOV calculation
			return GetHorizontalFieldOfView();
		}
		/** The background will be scaled & cropped to the screen */
		case EAppleARKitBackgroundFitMode::Fill:
		{
			// Is the screen wider than the camera (is the camera taller than the screen)?
			if ( ScreenAspectRatio > CameraAspectRatio )
			{
				// The camera texture will be scaled uniformly to fill the width of the screen, 
				// leaving the full horizonatal FOV visibile
				return GetHorizontalFieldOfView();
			} 
			// The camera is wider than the screen (the screen is taller than the camera)
			else
			{
				// The camera texture will be scaled uniformly to fill the height of the screen.
				// As the camera is wider than the screen, the camera image will extend beyond 
				// the sides of the screen, thus cropping the image & FOV to the screen aspect 
				// ratio.
				const float CroppedImageWidth = ImageResolution.Y * ScreenAspectRatio;
				return FMath::RadiansToDegrees( 2.0f * FMath::Atan( ( CroppedImageWidth / 2.0f ) / FocalLength.X ) );
			}
		}	
		/** The background image will be stretched to fill the screen */
		case EAppleARKitBackgroundFitMode::Stretch:
		{
			return GetHorizontalFieldOfView();
		}
	}

	// Fallback
	return GetHorizontalFieldOfView();
}

float FAppleARKitCamera::GetVerticalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode, float ScreenWidth, float ScreenHeight ) const
{
	// Sanity check?
	if ( FocalLength.Y <= 0.0f )
	{
		return 0.0f;
	}
	
	// Are they the same aspect ratio anyway?
	float ScreenAspectRatio = ScreenWidth / ScreenHeight;
	float CameraAspectRatio = 1.0f / GetAspectRatio();
	if (FMath::IsNearlyEqual(ScreenAspectRatio, CameraAspectRatio))
	{
		return GetVerticalFieldOfView();
	}
	
	// Not matching, figure out FOV for fit mode
	switch ( BackgroundFitMode )
	{
			/** The background image will be letterboxed to fit the screen */
		case EAppleARKitBackgroundFitMode::Fit:
		{
			return GetVerticalFieldOfView();
		}
			/** The background will be scaled & cropped to the screen */
		case EAppleARKitBackgroundFitMode::Fill:
		{
			// Is the screen wider than the camera (is the camera taller than the screen)?
			if ( ScreenAspectRatio > CameraAspectRatio )
			{
				// The camera texture will be scaled uniformly to fill the width of the screen,
				// leaving the full horizonatal FOV visibile
				return GetVerticalFieldOfView();
			}
			// The camera is wider than the screen (the screen is taller than the camera)
			else
			{
				// The camera texture will be scaled uniformly to fill the height of the screen.
				// As the camera is wider than the screen, the camera image will extend beyond
				// the sides of the screen, thus cropping the image & FOV to the screen aspect
				// ratio.
				const float CroppedImageWidth = ImageResolution.X * ScreenAspectRatio;
				return FMath::RadiansToDegrees( 2.0f * FMath::Atan( ( CroppedImageWidth / 2.0f ) / FocalLength.Y ) );
			}
		}
			/** The background image will be stretched to fill the screen */
		case EAppleARKitBackgroundFitMode::Stretch:
		{
			return GetVerticalFieldOfView();
		}
	}
	
	// Fallback
	return GetVerticalFieldOfView();
}

void FAppleARKitCamera::GetViewProjectionMatrix(EDeviceScreenOrientation DeviceOrientation, FSceneViewProjectionData& InOutProjectionData) const
{
	// Use the global viewport size as the screen size
	FVector2D ViewportSize;
	GEngine->GameViewport->GetViewportSize( ViewportSize );
	
	float FOV = 0.0f;
	if (DeviceOrientation == EDeviceScreenOrientation::Portrait || DeviceOrientation == EDeviceScreenOrientation::PortraitUpsideDown)
	{
		// Portrait
		FOV = GetVerticalFieldOfViewForScreen(EAppleARKitBackgroundFitMode::Fill);
	}
	else
	{
		// Landscape
		FOV = GetHorizontalFieldOfViewForScreen(EAppleARKitBackgroundFitMode::Fill);
	}
	
	InOutProjectionData.ProjectionMatrix = FReversedZPerspectiveMatrix(FMath::DegreesToRadians(FOV) * 0.5f, ViewportSize.X, ViewportSize.Y, 10.0f);
}


FVector2D FAppleARKitCamera::GetImageCoordinateForScreenPosition( FVector2D ScreenPosition, EAppleARKitBackgroundFitMode BackgroundFitMode, float ScreenWidth, float ScreenHeight ) const
{
	const bool bIsInPortraitMode = ScreenWidth < ScreenHeight;
	
	// Are they the same aspect ratio anyway?
	float ScreenAspectRatio = ScreenWidth / ScreenHeight;
	float CameraAspectRatio = bIsInPortraitMode ? 1.0f / GetAspectRatio() : GetAspectRatio();
	if (FMath::IsNearlyEqual(ScreenAspectRatio, CameraAspectRatio))
	{
		// Normalize ScreenPosition by viewport size
		return FVector2D(ScreenPosition.X / ScreenWidth, ScreenPosition.Y / ScreenHeight);
	}

	// Not matching, figure out screen pos for fit mode
	switch ( BackgroundFitMode )
	{
		case EAppleARKitBackgroundFitMode::Fill:
		{
			// Is the screen wider than the camera (is the camera taller than the screen)?
			if ( ScreenAspectRatio > CameraAspectRatio )
			{
				// The camera texture will be scaled uniformly to fill the width of the screen.
				// As the camera is taller than the screen, the camera image will extend beyond 
				// the top and bottom of the screen, thus cropping the image to the screen aspect 
				// ratio.
				float ImageOnScreenHeight = ScreenWidth / CameraAspectRatio;
				return FVector2D(
					ScreenPosition.X / ScreenWidth, 
					( ( ImageOnScreenHeight - ScreenHeight ) / 2.0f + ScreenPosition.Y ) / ImageOnScreenHeight );
			} 
			// The camera is wider than the screen (the screen is taller than the camera)
			else
			{
				// The camera texture will be scaled uniformly to fill the height of the screen.
				// As the camera is wider than the screen, the camera image will extend beyond 
				// the sides of the screen, thus cropping the image to the screen aspect 
				// ratio.
				float ImageOnScreenWidth = ScreenHeight * CameraAspectRatio;
				return FVector2D(
					( ( ImageOnScreenWidth - ScreenWidth ) / 2.0f + ScreenPosition.X ) / ImageOnScreenWidth,
					ScreenPosition.Y / ScreenHeight ); 
			}
		}
	}

	// Fallback to screen space
	return FVector2D( ScreenPosition.X / ScreenWidth, ScreenPosition.Y / ScreenHeight );
}
