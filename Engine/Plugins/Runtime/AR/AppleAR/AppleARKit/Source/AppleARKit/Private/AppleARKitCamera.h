// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE
#include "Math/Transform.h"
#include "Math/UnrealMathUtility.h"
#include "UnrealEngine.h"
#include "Engine/GameViewportClient.h"
#include "AppleARKitAvailability.h"
#include "ARSystem.h"

// ARKit
#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

#include "AppleARKitCamera.generated.h"

struct FSceneViewProjectionData;


enum class EAppleARKitBackgroundFitMode : uint8
{
	/** The background image will be letterboxed to fit the screen */
	Fit,

	/** The background will be scaled & cropped to the screen */
	Fill,

	/** The background image will be stretched to fill the screen */
	Stretch,
};


/**
 * A model representing the camera and its properties at a single point in time.
 */
USTRUCT( Category="AppleARKit" )
struct APPLEARKIT_API FAppleARKitCamera
{
	GENERATED_BODY()
	
	// Default constructor
	FAppleARKitCamera()
		: TrackingQuality(EARTrackingQuality::NotTracking)
		, TrackingQualityReason(EARTrackingQualityReason::None)
		, Orientation(ForceInit)
		, Translation(ForceInitToZero)
		, ImageResolution(ForceInitToZero)
		, FocalLength(ForceInitToZero)
		, PrincipalPoint(ForceInitToZero)
	{};

#if SUPPORTS_ARKIT_1_0

	/** 
	 * This is a conversion copy-constructor that takes a raw ARCamera and fills this structs members
	 * with the UE-ified versions of ARCamera's properties.
	 */ 
	FAppleARKitCamera( ARCamera* InARCamera );

#endif

	/**
	 * The tracking quality of the camera.
	 */
	UPROPERTY()
	EARTrackingQuality TrackingQuality;
	
	/**
	 * The reason for the current tracking quality of the camera.
	 */
	UPROPERTY()
	EARTrackingQualityReason TrackingQualityReason;

	/**
	 * The transformation matrix that defines the camera's rotation and translation in world coordinates.
	 */
	UPROPERTY()
	FTransform Transform;

	/* Raw orientation of the camera */
	UPROPERTY()
	FQuat Orientation;

	/* Raw position of the camera */
	UPROPERTY()
	FVector Translation;

	/**
	 * Camera image resolution in pixels
	 */
	UPROPERTY()
	FVector2D ImageResolution;

	/**
	 * Camera focal length in pixels
	 */
	UPROPERTY()
	FVector2D FocalLength;

	/**
	 * Camera principal point in pixels
	 */
	UPROPERTY()
	FVector2D PrincipalPoint;

	/** Returns the ImageResolution aspect ration (Width / Height) */
	float GetAspectRatio() const;

	/** Returns the horizonal FOV of the camera on this frame in degrees. */
	float GetHorizontalFieldOfView() const;

	/** Returns the vertical FOV of the camera on this frame in degrees. */
	float GetVerticalFieldOfView() const;

	/** Returns the effective horizontal field of view for the screen dimensions and fit mode in those dimensions */
	float GetHorizontalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode ) const;
	
	/** Returns the effective horizontal field of view for the screen when a device is in portrait mode */
	float GetVerticalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode ) const;

	/** Returns the effective horizontal field of view for the screen dimensions and fit mode in those dimensions */
	float GetHorizontalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode, float ScreenWidth, float ScreenHeight ) const;
	
	/** Returns the effective vertical field of view for the screen dimensions and fit mode in those dimensions */
	float GetVerticalFieldOfViewForScreen( EAppleARKitBackgroundFitMode BackgroundFitMode, float ScreenWidth, float ScreenHeight ) const;

	/** Returns the Projection Matrix we need to render our virtual content with. */
	void GetViewProjectionMatrix(EDeviceScreenOrientation DeviceOrientation, FSceneViewProjectionData& InOutProjectionData) const;
	
	

	/** For the given screen position, returns the normalised capture image coordinates accounting for the fit mode of the image on screen */
	FVector2D GetImageCoordinateForScreenPosition( FVector2D ScreenPosition, EAppleARKitBackgroundFitMode BackgroundFitMode ) const
	{
		// Use the global viewport size as the screen size
		FVector2D ViewportSize;
		GEngine->GameViewport->GetViewportSize( ViewportSize );

		return GetImageCoordinateForScreenPosition( ScreenPosition, BackgroundFitMode, ViewportSize.X, ViewportSize.Y );
	}

	/** For the given screen position, returns the normalised capture image coordinates accounting for the fit mode of the image on screen */
	FVector2D GetImageCoordinateForScreenPosition( FVector2D ScreenPosition, EAppleARKitBackgroundFitMode BackgroundFitMode, float ScreenWidth, float ScreenHeight ) const;
};
