// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitConversion.h"
#include "AppleARKitModule.h"
#include "HAL/PlatformMisc.h"
#include "AppleARKitFaceSupport.h"

#if PLATFORM_IOS
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSView.h"
#endif

template<typename TEnum>
static FString GetEnumValueAsString(const TCHAR* PathName, TEnum Value)
{
	if (const UEnum* EnumClass = FindObject<UEnum>(nullptr, PathName, EFindObjectFlags::ExactClass))
	{
		return EnumClass->GetNameByValue((int64)Value).ToString();
	}

	return FString("Invalid");
}

static IAppleARKitFaceSupport* GetFaceARSupport()
{
	auto Implementation = static_cast<IAppleARKitFaceSupport*>(IModularFeatures::Get().GetModularFeatureImplementation(IAppleARKitFaceSupport::GetModularFeatureName(), 0));
	return Implementation;
}

#if SUPPORTS_ARKIT_1_0
ARWorldAlignment FAppleARKitConversion::ToARWorldAlignment( const EARWorldAlignment& InWorldAlignment )
{
	switch ( InWorldAlignment )
	{
		case EARWorldAlignment::Gravity:
			return ARWorldAlignmentGravity;

		case EARWorldAlignment::GravityAndHeading:
			return ARWorldAlignmentGravityAndHeading;

		case EARWorldAlignment::Camera:
			return ARWorldAlignmentCamera;
	};
};

#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wpartial-availability"

#if SUPPORTS_ARKIT_1_5
static float GetAspectRatio(float Width, float Height)
{
	if (Width > Height)
	{
		return Width / Height;
	}
	else
	{
		return Height / Width;
	}
}

ARVideoFormat* FAppleARKitConversion::ToARVideoFormat(const FARVideoFormat& DesiredFormat, NSArray<ARVideoFormat*>* Formats, bool bUseOptimalFormat)
{
	if (Formats != nullptr)
	{
		for (ARVideoFormat* Format in Formats)
		{
			if (Format != nullptr &&
				DesiredFormat.FPS == Format.framesPerSecond &&
				DesiredFormat.Width == Format.imageResolution.width &&
				DesiredFormat.Height == Format.imageResolution.height)
			{
				return Format;
			}
		}
		
		if (bUseOptimalFormat)
		{
			auto View = [IOSAppDelegate GetDelegate].IOSView;
			auto Frame = [View frame];
			const auto FrameAspectRatio = GetAspectRatio(Frame.size.width, Frame.size.height);
			ARVideoFormat* BestFormat = nullptr;
			float BestAspectRatioDiff = FLT_MAX;
			float BestImageWidth = -1.f;
			for (ARVideoFormat* Format in Formats)
			{
				if (!Format)
				{
					continue;
				}
				
				// This is the diff between the format aspect ratio and the desired aspect ratio
				const auto AspectRatioDiff = FMath::Abs(GetAspectRatio(Format.imageResolution.width, Format.imageResolution.height) - FrameAspectRatio);
				if ((AspectRatioDiff < BestAspectRatioDiff) || // Pick the format if it matches the desired one better
					(AspectRatioDiff == BestAspectRatioDiff && Format.imageResolution.width > BestImageWidth)) // Or if it provides better image quality
				{
					BestAspectRatioDiff = AspectRatioDiff;
					BestFormat = Format;
					BestImageWidth = Format.imageResolution.width;
				}
			}
			
			if (BestFormat)
			{
				UE_LOGF(LogAppleARKit, Log, "Selected optimal video format (%.0f x %.0f) to match screen size (%.0f x %.0f)",
					   BestFormat.imageResolution.width, BestFormat.imageResolution.height, Frame.size.width, Frame.size.height);
				return BestFormat;
			}
		}
	}
	return nullptr;
}

FARVideoFormat FAppleARKitConversion::FromARVideoFormat(ARVideoFormat* Format)
{
	FARVideoFormat ConvertedFormat;
	if (Format != nullptr)
	{
		ConvertedFormat.FPS = Format.framesPerSecond;
		ConvertedFormat.Width = Format.imageResolution.width;
		ConvertedFormat.Height = Format.imageResolution.height;
	}
	return ConvertedFormat;
}

TArray<FARVideoFormat> FAppleARKitConversion::FromARVideoFormatArray(NSArray<ARVideoFormat*>* Formats)
{
	TArray<FARVideoFormat> ConvertedArray;
	if (Formats != nullptr)
	{
		for (ARVideoFormat* Format in Formats)
		{
			if (Format != nullptr)
			{
				ConvertedArray.Add(FromARVideoFormat(Format));
			}
		}
	}
	return ConvertedArray;
}

NSSet* FAppleARKitConversion::InitImageDetection(UARSessionConfig* SessionConfig, TMap< FString, UARCandidateImage* >& CandidateImages, TMap< FString, CGImageRef >& ConvertedCandidateImages)
{
	const TArray<UARCandidateImage*>& ConfigCandidateImages = SessionConfig->GetCandidateImageList();
	if (!ConfigCandidateImages.Num())
	{
		return nullptr;
	}

	NSMutableSet* ConvertedImageSet = [[NSMutableSet new] autorelease];
	for (UARCandidateImage* Candidate : ConfigCandidateImages)
	{
		if (Candidate != nullptr && Candidate->GetCandidateTexture() != nullptr)
		{
			// Don't crash if the physical size is invalid
			if (Candidate->GetPhysicalWidth() <= 0.f || Candidate->GetPhysicalHeight() <= 0.f)
			{
				UE_LOGF(LogAppleARKit, Error, "Unable to process candidate image (%ls - %ls) due to an invalid physical size (%f,%f)",
				   *Candidate->GetFriendlyName(), *Candidate->GetName(), Candidate->GetPhysicalWidth(), Candidate->GetPhysicalHeight());
				continue;
			}
			// Store off so the session object can quickly match the anchor to our representation
			// This stores it even if we weren't able to convert to apple's type for GC reasons
			CandidateImages.Add(Candidate->GetFriendlyName(), Candidate);
			// Convert our texture to an Apple compatible image type
			CGImageRef ConvertedImage = nullptr;
			// Avoid doing the expensive conversion work if it's in the cache already
			CGImageRef* FoundImage = ConvertedCandidateImages.Find(Candidate->GetFriendlyName());
			if (FoundImage != nullptr)
			{
				ConvertedImage = *FoundImage;
			}
			else
			{
				ConvertedImage = IAppleImageUtilsPlugin::Get().UTexture2DToCGImage(Candidate->GetCandidateTexture());
				// If it didn't convert this time, it never will, so always store it off
				ConvertedCandidateImages.Add(Candidate->GetFriendlyName(), ConvertedImage);
			}
			if (ConvertedImage != nullptr)
			{
				float ImageWidth = (float)Candidate->GetPhysicalWidth() / 100.f;
				ARReferenceImage* ReferenceImage = [[[ARReferenceImage alloc] initWithCGImage: ConvertedImage orientation: kCGImagePropertyOrientationUp physicalWidth: ImageWidth] autorelease];
				ReferenceImage.name = Candidate->GetFriendlyName().GetNSString();
				[ConvertedImageSet addObject: ReferenceImage];
			}
		}
	}
	return ConvertedImageSet;
}
#endif

#if SUPPORTS_ARKIT_2_0
void FAppleARKitConversion::InitImageDetection(UARSessionConfig* SessionConfig, ARImageTrackingConfiguration* ImageConfig, TMap< FString, UARCandidateImage* >& CandidateImages, TMap< FString, CGImageRef >& ConvertedCandidateImages)
{
	ImageConfig.trackingImages = InitImageDetection(SessionConfig, CandidateImages, ConvertedCandidateImages);
	ImageConfig.maximumNumberOfTrackedImages = SessionConfig->GetMaxNumSimultaneousImagesTracked();
}

AREnvironmentTexturing FAppleARKitConversion::ToAREnvironmentTexturing(EAREnvironmentCaptureProbeType CaptureType)
{
	switch (CaptureType)
	{
		case EAREnvironmentCaptureProbeType::Manual:
		{
			return AREnvironmentTexturingManual;
		}
		case EAREnvironmentCaptureProbeType::Automatic:
		{
			return AREnvironmentTexturingAutomatic;
		}
	}
	return AREnvironmentTexturingNone;
}

ARWorldMap* FAppleARKitConversion::ToARWorldMap(const TArray<uint8>& WorldMapData, TOptional<FTransform>& InAlignmentTransform)
{
	uint8* Buffer = (uint8*)WorldMapData.GetData();
	FARWorldSaveHeader InHeader(Buffer);
	
	// Check for our format and reject if invalid
	if (InHeader.Magic != AR_SAVE_WORLD_KEY || InHeader.Version > (uint8)EARSaveWorldVersions::Latest)
	{
		UE_LOGF(LogAppleARKit, Log, "Failed to load the world map data from the session object due to incompatible versions: magic (0x%x), ver(%d)", InHeader.Magic, (uint32)InHeader.Version);
		return nullptr;
	}
	
	uint32 HeaderAndAlignmentSize = 0;
	auto AlignmentTransform = FTransform::Identity;
	// Copy out the alignment transform if the data contains it
	if (InHeader.Version >= (uint8)EARSaveWorldVersions::Latest)
	{
		AlignmentTransform = *(FTransform*)(Buffer + AR_SAVE_WORLD_HEADER_SIZE);
		HeaderAndAlignmentSize = AR_SAVE_WORLD_HEADER_SIZE + sizeof(FTransform);
	}
	else
	{
		HeaderAndAlignmentSize = AR_SAVE_WORLD_HEADER_SIZE;
	}
	
	// Decompress the data, see FAppleARKitSaveWorldAsyncTask::OnWorldMapAcquired for how it's encoded
	uint8* CompressedData = Buffer + HeaderAndAlignmentSize;
	uint32 CompressedSize = WorldMapData.Num() - HeaderAndAlignmentSize;
	uint32 UncompressedSize = InHeader.UncompressedSize;
	TArray<uint8> UncompressedData;
	UncompressedData.AddUninitialized(UncompressedSize);
	if (!FCompression::UncompressMemory(NAME_Zlib, UncompressedData.GetData(), UncompressedSize, CompressedData, CompressedSize))
	{
		UE_LOGF(LogAppleARKit, Log, "Failed to load the world map data from the session object due to a decompression error");
		return nullptr;
	}
	
	// Serialize into the World map data
	NSData* WorldNSData = [NSData dataWithBytesNoCopy: UncompressedData.GetData() length: UncompressedData.Num() freeWhenDone: NO];
	NSError* ErrorObj = nullptr;
	ARWorldMap* WorldMap = [NSKeyedUnarchiver unarchivedObjectOfClass: ARWorldMap.class fromData: WorldNSData error: &ErrorObj];
	if (ErrorObj != nullptr)
	{
		FString Error = [ErrorObj localizedDescription];
		UE_LOGF(LogAppleARKit, Log, "Failed to load the world map data from the session object with error string (%ls)", *Error);
	}
	InAlignmentTransform = AlignmentTransform;
	return WorldMap;
}

NSSet* FAppleARKitConversion::ToARReferenceObjectSet(const TArray<UARCandidateObject*>& CandidateObjects, TMap< FString, UARCandidateObject* >& CandidateObjectMap)
{
	CandidateObjectMap.Empty();

	if (CandidateObjects.Num() == 0)
	{
		return nullptr;
	}

	NSMutableSet* ConvertedObjectSet = [[NSMutableSet new] autorelease];
	for (UARCandidateObject* Candidate : CandidateObjects)
	{
		if (Candidate != nullptr && Candidate->GetCandidateObjectData().Num() > 0)
		{
			NSData* CandidateData = [NSData dataWithBytesNoCopy: (uint8*)Candidate->GetCandidateObjectData().GetData() length: Candidate->GetCandidateObjectData().Num() freeWhenDone: NO];
			NSError* ErrorObj = nullptr;
			ARReferenceObject* RefObject = [NSKeyedUnarchiver unarchivedObjectOfClass: ARReferenceObject.class fromData: CandidateData error: &ErrorObj];
			if (RefObject != nullptr)
			{
				// Store off so the session object can quickly match the anchor to our representation
				// This stores it even if we weren't able to convert to apple's type for GC reasons
				CandidateObjectMap.Add(Candidate->GetFriendlyName(), Candidate);
				RefObject.name = Candidate->GetFriendlyName().GetNSString();
				[ConvertedObjectSet addObject: RefObject];
			}
			else
			{
				UE_LOGF(LogAppleARKit, Log, "Failed to convert to ARReferenceObject (%ls)", *Candidate->GetFriendlyName());
			}
		}
		else
		{
			UE_LOGF(LogAppleARKit, Log, "Missing candidate object data for ARCandidateObject (%ls)", Candidate != nullptr ? *Candidate->GetFriendlyName() : TEXT("null"));
		}
	}
	return ConvertedObjectSet;
}
#endif

#if SUPPORTS_ARKIT_1_5
NSArray<ARVideoFormat*>* FAppleARKitConversion::GetSupportedVideoFormats(EARSessionType SessionType)
{
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		switch (SessionType)
		{
			case EARSessionType::Orientation:
				return AROrientationTrackingConfiguration.supportedVideoFormats;
				
			case EARSessionType::World:
				return ARWorldTrackingConfiguration.supportedVideoFormats;
				
			case EARSessionType::Face:
				if (auto Implementation = GetFaceARSupport())
				{
					return Implementation->GetSupportedVideoFormats();
				}
				break;
				
#if SUPPORTS_ARKIT_2_0
			case EARSessionType::Image:
				if (FAppleARKitAvailability::SupportsARKit20())
				{
					return ARImageTrackingConfiguration.supportedVideoFormats;
				}
				break;
				
			case EARSessionType::ObjectScanning:
				if (FAppleARKitAvailability::SupportsARKit20())
				{
					return ARObjectScanningConfiguration.supportedVideoFormats;
				}
				break;
#endif
				
#if SUPPORTS_ARKIT_3_0
			case EARSessionType::PoseTracking:
				if (FAppleARKitAvailability::SupportsARKit30())
				{
					return ARBodyTrackingConfiguration.supportedVideoFormats;
				}
				break;
#endif
		}
	}
	return nullptr;
}
#endif

template<class T>
static void ConfigureAutoFocus(UARSessionConfig* SessionConfig, T* ARConfig)
{
#if SUPPORTS_ARKIT_1_5
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		ARConfig.autoFocusEnabled = SessionConfig->ShouldEnableAutoFocus();
	}
#endif
}

template<class T>
static void ConfigurePlaneDetection(UARSessionConfig* SessionConfig, T* ARConfig)
{
	ARConfig.planeDetection = ARPlaneDetectionNone;
	if (EnumHasAnyFlags(EARPlaneDetectionMode::HorizontalPlaneDetection, SessionConfig->GetPlaneDetectionMode()))
	{
		ARConfig.planeDetection |= ARPlaneDetectionHorizontal;
	}
#if SUPPORTS_ARKIT_1_5
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		if (EnumHasAnyFlags(EARPlaneDetectionMode::VerticalPlaneDetection, SessionConfig->GetPlaneDetectionMode()))
		{
			ARConfig.planeDetection |= ARPlaneDetectionVertical;
		}
	}
#endif
}

template<class T>
static void ConfigureEnvironmentTexturing(UARSessionConfig* SessionConfig, T* ARConfig)
{
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		ARConfig.environmentTexturing = FAppleARKitConversion::ToAREnvironmentTexturing(SessionConfig->GetEnvironmentCaptureProbeType());
	}
#endif
	
#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		// Disable HDR environment texturing as the engine doesn't support it yet
		// See FARMetalResource in AppleARKitTextures.cpp
		// TODO: Add HDR support later
		ARConfig.wantsHDREnvironmentTextures = NO;
	}
#endif
}

template<class T>
static void ConfigureInitialWorldMap(UARSessionConfig* SessionConfig, T* ARConfig, TOptional<FTransform>& InitialAlignmentTransform)
{
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		// Load the world if requested
		if (SessionConfig->GetWorldMapData().Num() > 0)
		{
			ARWorldMap* WorldMap = FAppleARKitConversion::ToARWorldMap(SessionConfig->GetWorldMapData(), InitialAlignmentTransform);
			ARConfig.initialWorldMap = WorldMap;
			[WorldMap release];
		}
	}
#endif
}

template<class T>
static void ConfigureObjectDetection(UARSessionConfig* SessionConfig, T* ARConfig, TMap<FString, UARCandidateObject*>& CandidateObjects)
{
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		// Convert any candidate objects that are to be detected
		ARConfig.detectionObjects = FAppleARKitConversion::ToARReferenceObjectSet(SessionConfig->GetCandidateObjectList(), CandidateObjects);
	}
#endif
}

template<class T>
static void ConfigureImageDetection(UARSessionConfig* SessionConfig, T* ARConfig, TMap<FString, UARCandidateImage*>& CandidateImages, TMap<FString, CGImageRef>& ConvertedCandidateImages)
{
#if SUPPORTS_ARKIT_1_5
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		ARConfig.detectionImages = FAppleARKitConversion::InitImageDetection(SessionConfig, CandidateImages, ConvertedCandidateImages);
	}
#endif
		
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		ARConfig.maximumNumberOfTrackedImages = SessionConfig->GetMaxNumSimultaneousImagesTracked();
	}
#endif
	
#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		ARConfig.automaticImageScaleEstimationEnabled = SessionConfig->bUseAutomaticImageScaleEstimation;
	}
#endif
}

ARConfiguration* FAppleARKitConversion::ToARConfiguration(UARSessionConfig* SessionConfig, TMap<FString, UARCandidateImage*>& CandidateImages, TMap<FString, CGImageRef>& ConvertedCandidateImages, TMap<FString, UARCandidateObject*>& CandidateObjects, TOptional<FTransform>& InitialAlignmentTransform)
{
	EARSessionType SessionType = SessionConfig->GetSessionType();
	ARConfiguration* SessionConfiguration = nullptr;
	switch (SessionType)
	{
		case EARSessionType::Orientation:
		{
			if (AROrientationTrackingConfiguration.isSupported == FALSE)
			{
				return nullptr;
			}
			AROrientationTrackingConfiguration* OrientationTrackingConfiguration = [AROrientationTrackingConfiguration new];
			ConfigureAutoFocus(SessionConfig, OrientationTrackingConfiguration);
			SessionConfiguration = OrientationTrackingConfiguration;
			break;
		}

		case EARSessionType::World:
		{
			if (ARWorldTrackingConfiguration.isSupported == FALSE)
			{
				return nullptr;
			}
			ARWorldTrackingConfiguration* WorldTrackingConfiguration = [ARWorldTrackingConfiguration new];
			ConfigureAutoFocus(SessionConfig, WorldTrackingConfiguration);
			ConfigurePlaneDetection(SessionConfig, WorldTrackingConfiguration);
			ConfigureImageDetection(SessionConfig, WorldTrackingConfiguration, CandidateImages, ConvertedCandidateImages);
			ConfigureEnvironmentTexturing(SessionConfig, WorldTrackingConfiguration);
			ConfigureInitialWorldMap(SessionConfig, WorldTrackingConfiguration, InitialAlignmentTransform);
			ConfigureObjectDetection(SessionConfig, WorldTrackingConfiguration, CandidateObjects);

#if SUPPORTS_ARKIT_3_5
			if (FAppleARKitAvailability::SupportsARKit35())
			{
				// Configure the scene reconstruction method
				const auto SceneReconstructionMethod = SessionConfig->GetSceneReconstructionMethod();
				if (SceneReconstructionMethod != EARSceneReconstruction::None)
				{
					if (IsSceneReconstructionSupported(EARSessionType::World, SceneReconstructionMethod))
					{
						static const TMap<EARSceneReconstruction, ARSceneReconstruction> Mapping =
						{
							{ EARSceneReconstruction::MeshOnly, ARSceneReconstructionMesh },
							{ EARSceneReconstruction::MeshWithClassification, ARSceneReconstructionMeshWithClassification }
						};
						
						WorldTrackingConfiguration.sceneReconstruction = Mapping[SceneReconstructionMethod];
					}
				}
			}
#endif
			SessionConfiguration = WorldTrackingConfiguration;
			break;
		}
	
#if SUPPORTS_ARKIT_2_0
		case EARSessionType::Image:
		{
			if (FAppleARKitAvailability::SupportsARKit20())
			{
				if (ARImageTrackingConfiguration.isSupported == FALSE)
				{
					return nullptr;
				}
				ARImageTrackingConfiguration* ImageTrackingConfiguration = [ARImageTrackingConfiguration new];
				ConfigureAutoFocus(SessionConfig, ImageTrackingConfiguration);
				
				// Add any images that wish to be detected
				InitImageDetection(SessionConfig, ImageTrackingConfiguration, CandidateImages, ConvertedCandidateImages);
				SessionConfiguration = ImageTrackingConfiguration;
			}
			break;
		}

		case EARSessionType::ObjectScanning:
		{
			if (FAppleARKitAvailability::SupportsARKit20())
			{
				if (ARObjectScanningConfiguration.isSupported == FALSE)
				{
					return nullptr;
				}
				ARObjectScanningConfiguration* ObjectScanningConfiguration = [ARObjectScanningConfiguration new];
				ConfigureAutoFocus(SessionConfig, ObjectScanningConfiguration);
				ConfigurePlaneDetection(SessionConfig, ObjectScanningConfiguration);
				SessionConfiguration = ObjectScanningConfiguration;
			}
			break;
		}
#endif // SUPPORTS_ARKIT_2_0

#if SUPPORTS_ARKIT_3_0
		case EARSessionType::PoseTracking:
		{
			if (FAppleARKitAvailability::SupportsARKit30())
			{
				if (ARBodyTrackingConfiguration.isSupported == FALSE)
				{
					return nullptr;
				}
				
				ARBodyTrackingConfiguration* BodyTrackingConfiguration = [ARBodyTrackingConfiguration new];
				ConfigureAutoFocus(SessionConfig, BodyTrackingConfiguration);
				ConfigurePlaneDetection(SessionConfig, BodyTrackingConfiguration);
				ConfigureImageDetection(SessionConfig, BodyTrackingConfiguration, CandidateImages, ConvertedCandidateImages);
				ConfigureEnvironmentTexturing(SessionConfig, BodyTrackingConfiguration);
				ConfigureInitialWorldMap(SessionConfig, BodyTrackingConfiguration, InitialAlignmentTransform);
				SessionConfiguration = BodyTrackingConfiguration;
			}
			break;
		}
#endif // SUPPORTS_ARKIT_3_0

#if SUPPORTS_ARKIT_4_0
		case EARSessionType::GeoTracking:
		{
			if (FAppleARKitAvailability::SupportsARKit40())
			{
				if (ARGeoTrackingConfiguration.isSupported == FALSE)
				{
					return nullptr;
				}
				
				ARGeoTrackingConfiguration* GeoTrackingConfiguration = [ARGeoTrackingConfiguration new];
				ConfigurePlaneDetection(SessionConfig, GeoTrackingConfiguration);
				ConfigureImageDetection(SessionConfig, GeoTrackingConfiguration, CandidateImages, ConvertedCandidateImages);
				ConfigureEnvironmentTexturing(SessionConfig, GeoTrackingConfiguration);
				SessionConfiguration = GeoTrackingConfiguration;
			}
			break;
		}
#endif // SUPPORTS_ARKIT_4_0
	}
	
	if (SessionConfiguration != nullptr)
	{
		// Copy / convert properties
		SessionConfiguration.lightEstimationEnabled = SessionConfig->GetLightEstimationMode() != EARLightEstimationMode::None;
		SessionConfiguration.providesAudioData = NO;
		SessionConfiguration.worldAlignment = FAppleARKitConversion::ToARWorldAlignment(SessionConfig->GetWorldAlignment());
		
		if (auto SupportedFormats = FAppleARKitConversion::GetSupportedVideoFormats(SessionType))
		{
			if (auto VideoFormat = FAppleARKitConversion::ToARVideoFormat(SessionConfig->GetDesiredVideoFormat(), SupportedFormats, SessionConfig->ShouldUseOptimalVideoFormat()))
			{
				SessionConfiguration.videoFormat = VideoFormat;
			}
		}
	}
	
	return SessionConfiguration;
}

void FAppleARKitConversion::ConfigureSessionTrackingFeatures(UARSessionConfig* SessionConfig, ARConfiguration* SessionConfiguration)
{
#if SUPPORTS_ARKIT_3_0
		// Enable additional frame semantics for ARKit 3.0
		if (FAppleARKitAvailability::SupportsARKit30())
		{
			EARSessionType SessionType = SessionConfig->GetSessionType();
			const EARSessionTrackingFeature SessionTrackingFeature = SessionConfig->GetEnabledSessionTrackingFeature();
			if (SessionTrackingFeature != EARSessionTrackingFeature::None)
			{
				if (IsSessionTrackingFeatureSupported(SessionType, SessionTrackingFeature))
				{
					SessionConfiguration.frameSemantics = ToARFrameSemantics(SessionTrackingFeature);
				}
				else
				{
					UE_LOGF(LogAppleARKit, Error, "Session type [%ls] doesn't support the required session feature: [%ls]!",
						   *GetEnumValueAsString<>(TEXT("/Script/AugmentedReality.EARSessionType"), SessionType),
						   *GetEnumValueAsString<>(TEXT("/Script/AugmentedReality.EARSessionTrackingFeature"), SessionTrackingFeature));
				}
			}
		}
#endif
}

bool FAppleARKitConversion::IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature)
{
#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		const ARFrameSemantics Semantics = ToARFrameSemantics(SessionTrackingFeature);
		if (Semantics != ARFrameSemanticNone)
		{
			switch (SessionType)
			{
				case EARSessionType::Orientation:
					return [AROrientationTrackingConfiguration supportsFrameSemantics: Semantics];
				
				case EARSessionType::World:
					return [ARWorldTrackingConfiguration supportsFrameSemantics: Semantics];
				
				case EARSessionType::Face:
					{
						static TMap<EARSessionTrackingFeature, bool> SupportFlags;
						
						if (!SupportFlags.Contains(SessionTrackingFeature))
						{
							SupportFlags.Add(SessionTrackingFeature, false);
							
							if (auto Implementation = GetFaceARSupport())
							{
								if (Implementation->IsARFrameSemanticsSupported(Semantics))
								{
									SupportFlags[SessionTrackingFeature] = true;
								}
							}
						}
						
						return SupportFlags[SessionTrackingFeature];
					}
				
				case EARSessionType::Image:
					return [ARImageTrackingConfiguration supportsFrameSemantics: Semantics];
				
				case EARSessionType::ObjectScanning:
					return [ARObjectScanningConfiguration supportsFrameSemantics: Semantics];
					
				case EARSessionType::PoseTracking:
					return [ARBodyTrackingConfiguration supportsFrameSemantics: Semantics];
			}
		}
	}
#endif
	
	return false;
}

bool FAppleARKitConversion::IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod)
{
#if SUPPORTS_ARKIT_3_5
	if (FAppleARKitAvailability::SupportsARKit35())
	{
		if (SessionType == EARSessionType::World)
		{
			switch (SceneReconstructionMethod)
			{
				case EARSceneReconstruction::None:
					return true;
					
				case EARSceneReconstruction::MeshOnly:
					return [ARWorldTrackingConfiguration supportsSceneReconstruction: ARSceneReconstructionMesh];
					
				case EARSceneReconstruction::MeshWithClassification:
					return [ARWorldTrackingConfiguration supportsSceneReconstruction: ARSceneReconstructionMeshWithClassification];
			}
		}
	}
#endif
	return false;
}

#if SUPPORTS_ARKIT_3_0
ARFrameSemantics FAppleARKitConversion::ToARFrameSemantics(EARSessionTrackingFeature SessionTrackingFeature)
{
	static const TMap<EARSessionTrackingFeature, ARFrameSemantics> SessionTrackingFeatureToFrameSemantics =
	{
		{ EARSessionTrackingFeature::None, ARFrameSemanticNone },
		{ EARSessionTrackingFeature::PoseDetection2D, ARFrameSemanticBodyDetection },
		{ EARSessionTrackingFeature::PersonSegmentation, ARFrameSemanticPersonSegmentation },
		{ EARSessionTrackingFeature::PersonSegmentationWithDepth, ARFrameSemanticPersonSegmentationWithDepth },
#if SUPPORTS_ARKIT_4_0
		{ EARSessionTrackingFeature::SceneDepth, ARFrameSemanticSceneDepth },
		{ EARSessionTrackingFeature::SmoothedSceneDepth, ARFrameSemanticSmoothedSceneDepth },
#endif
	};
	
	if (auto Record = SessionTrackingFeatureToFrameSemantics.Find(SessionTrackingFeature))
	{
		return *Record;
	}
	return ARFrameSemanticNone;
}

void FAppleARKitConversion::ToSkeletonDefinition(const ARSkeletonDefinition* InARSkeleton, FARSkeletonDefinition& OutSkeletonDefinition)
{
	check(InARSkeleton);
	
	// TODO: these values should not change over time so they can be cached somewhere
	
	const int NumJoints = InARSkeleton.jointCount;
	
	OutSkeletonDefinition.NumJoints = NumJoints;
	OutSkeletonDefinition.JointNames.AddUninitialized(NumJoints);
	OutSkeletonDefinition.ParentIndices.AddUninitialized(NumJoints);
	
	for (int Index = 0; Index < NumJoints; ++Index)
	{
		OutSkeletonDefinition.JointNames[Index] = *FString(InARSkeleton.jointNames[Index]);
		OutSkeletonDefinition.ParentIndices[Index] = InARSkeleton.parentIndices[Index].intValue;
	}
}

FARPose2D FAppleARKitConversion::ToARPose2D(const ARBody2D* InARPose2D)
{
	check(InARPose2D);
	
    const EDeviceScreenOrientation ScreenOrientation = FPlatformMisc::GetDeviceOrientation();
	
	FARPose2D Pose2D;
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		const ARSkeleton2D* Skeleton2D = InARPose2D.skeleton;
		ToSkeletonDefinition(Skeleton2D.definition, Pose2D.SkeletonDefinition);
		const int NumJoints = Pose2D.SkeletonDefinition.NumJoints;
		
		Pose2D.JointLocations.AddUninitialized(NumJoints);
		Pose2D.IsJointTracked.AddUninitialized(NumJoints);
		
		for (int Index = 0; Index < NumJoints; ++Index)
		{
			const bool bIsTracked = [Skeleton2D isJointTracked: Index];
			Pose2D.IsJointTracked[Index] = bIsTracked;
			
            if (bIsTracked)
            {
                FVector2D OriginalLandmark(Skeleton2D.jointLandmarks[Index][0], Skeleton2D.jointLandmarks[Index][1]);
				
                switch (ScreenOrientation)
                {
                    case EDeviceScreenOrientation::Portrait:
                        Pose2D.JointLocations[Index] = FVector2D(1.f - OriginalLandmark.Y, OriginalLandmark.X);
                        break;
                        
                    case EDeviceScreenOrientation::PortraitUpsideDown:
                        Pose2D.JointLocations[Index] = FVector2D(OriginalLandmark.Y, OriginalLandmark.X);
                        break;
                        
                    case EDeviceScreenOrientation::LandscapeLeft:
                        Pose2D.JointLocations[Index] = FVector2D(1.f - OriginalLandmark.X, 1.f - OriginalLandmark.Y);
                        break;
                        
                    case EDeviceScreenOrientation::LandscapeRight:
                        Pose2D.JointLocations[Index] = OriginalLandmark;
                        break;
                        
                    default:
                        Pose2D.JointLocations[Index] = OriginalLandmark;
                }
            }
            else
            {
                Pose2D.JointLocations[Index] = FVector2D::ZeroVector;
            }
		}
	}
	
	return Pose2D;
}

FARPose3D FAppleARKitConversion::ToARPose3D(const ARSkeleton3D* Skeleton3D, bool bIdentityForUntracked)
{
	check(Skeleton3D);
	
	FARPose3D Pose3D;
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		ToSkeletonDefinition(Skeleton3D.definition, Pose3D.SkeletonDefinition);
		const int NumJoints = Pose3D.SkeletonDefinition.NumJoints;
		
		Pose3D.JointTransforms.AddUninitialized(NumJoints);
		Pose3D.IsJointTracked.AddUninitialized(NumJoints);
		Pose3D.JointTransformSpace = EARJointTransformSpace::Model;
		
		for (int Index = 0; Index < NumJoints; ++Index)
		{
			const bool bIsTracked = [Skeleton3D isJointTracked: Index];
			Pose3D.IsJointTracked[Index] = bIsTracked;
			if (bIsTracked || !bIdentityForUntracked)
			{
				Pose3D.JointTransforms[Index] = ToFTransform(Skeleton3D.jointModelTransforms[Index]);
			}
			else
			{
				Pose3D.JointTransforms[Index] = FTransform::Identity;
			}
		}
	}
	
	return Pose3D;
}

FARPose3D FAppleARKitConversion::ToARPose3D(const ARBodyAnchor* InARBodyAnchor)
{
	check(InARBodyAnchor);
	
	FARPose3D Pose3D;
	if (FAppleARKitAvailability::SupportsARKit30())
    {
		Pose3D = ToARPose3D(InARBodyAnchor.skeleton, true);
	}
	
	return Pose3D;
}

#endif

#pragma clang diagnostic pop
#endif
