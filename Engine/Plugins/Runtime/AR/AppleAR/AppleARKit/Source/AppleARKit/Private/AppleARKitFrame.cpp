// Copyright Epic Games, Inc. All Rights Reserved.

// AppleARKit
#include "AppleARKitFrame.h"
#include "AppleARKitModule.h"
#include "AppleARKitConversion.h"
#include "Misc/ScopeLock.h"


#if SUPPORTS_ARKIT_2_0
EARWorldMappingState ToEARWorldMappingState(ARWorldMappingStatus MapStatus)
{
	switch (MapStatus)
	{
		// These both mean more data is needed
		case ARWorldMappingStatusLimited:
		case ARWorldMappingStatusExtending:
			return EARWorldMappingState::StillMappingNotRelocalizable;

		case ARWorldMappingStatusMapped:
			return EARWorldMappingState::Mapped;
	}
	return EARWorldMappingState::NotAvailable;
}
#endif

#if SUPPORTS_ARKIT_1_0
FAppleARKitFrame::FAppleARKitFrame(ARFrame* InARFrame, const FVector2D MinCameraUV, const FVector2D MaxCameraUV, CVMetalTextureCacheRef MetalTextureCache)
	: Camera( InARFrame.camera )
	, LightEstimate( InARFrame.lightEstimate )
	, WorldMappingState(EARWorldMappingState::NotAvailable)
{
	// Sanity check
	check( InARFrame );

	// Copy timestamp
	Timestamp = InARFrame.timestamp;

	CameraImage = nullptr;
	CameraDepth = nullptr;

	if ( InARFrame.capturedImage )
	{
		CameraImage = InARFrame.capturedImage;
		CFRetain(CameraImage);
		
		// Update SizeX & Y
		CapturedYImageSize.X = CVPixelBufferGetWidthOfPlane( InARFrame.capturedImage, 0 );
		CapturedYImageSize.Y = CVPixelBufferGetHeightOfPlane( InARFrame.capturedImage, 0 );
		CapturedCbCrImageSize.X = CVPixelBufferGetWidthOfPlane( InARFrame.capturedImage, 1 );
		CapturedCbCrImageSize.Y = CVPixelBufferGetHeightOfPlane( InARFrame.capturedImage, 1 );
		
		// Create a metal texture from the CVPixelBufferRef. The CVMetalTextureRef will
		// be released in the FAppleARKitFrame destructor.
		// NOTE: On success, CapturedImage will be a new CVMetalTextureRef with a ref count of 1
		// 		 so we don't need to CFRetain it. The corresponding CFRelease is handled in
		//
		CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(nullptr, MetalTextureCache, InARFrame.capturedImage, nullptr, MTLPixelFormatR8Unorm, CapturedYImageSize.X, CapturedYImageSize.Y, /*PlaneIndex*/0, &CapturedYImage);
		check( Result == kCVReturnSuccess );
		check( CapturedYImage );
		check( CFGetRetainCount(CapturedYImage) == 1);
		
		Result = CVMetalTextureCacheCreateTextureFromImage(nullptr, MetalTextureCache, InARFrame.capturedImage, nullptr, MTLPixelFormatRG8Unorm, CapturedCbCrImageSize.X, CapturedCbCrImageSize.Y, /*PlaneIndex*/1, &CapturedCbCrImage);
		check( Result == kCVReturnSuccess );
		check( CapturedCbCrImage );
		check( CFGetRetainCount(CapturedCbCrImage) == 1);
	}

	if (InARFrame.capturedDepthData)
	{
		CameraDepth = InARFrame.capturedDepthData;
		CFRetain(CameraDepth);
	}

	NativeFrame = InARFrame;
	[NativeFrame retain];

#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		WorldMappingState = ToEARWorldMappingState(InARFrame.worldMappingStatus);
	}
#endif

#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		if (InARFrame.detectedBody)
		{
			Tracked2DPose = FAppleARKitConversion::ToARPose2D(InARFrame.detectedBody);
			
			// Convert the joint location from the normalized arkit camera space to UE's normalized screen space
			const FVector2D UVSize = MaxCameraUV - MinCameraUV;
			for (int Index = 0; Index < Tracked2DPose.JointLocations.Num(); ++Index)
			{
				if (Tracked2DPose.IsJointTracked[Index])
				{
					FVector2D& JointLocation = Tracked2DPose.JointLocations[Index];
					JointLocation = (JointLocation - MinCameraUV) / UVSize;
				}
			}
		}
		
		if (InARFrame.segmentationBuffer)
		{
			SegmentationBuffer = InARFrame.segmentationBuffer;
			CFRetain(SegmentationBuffer);
		}
		
		if (InARFrame.estimatedDepthData)
		{
			EstimatedDepthData = InARFrame.estimatedDepthData;
			CFRetain(EstimatedDepthData);
		}
	}
#endif
	
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		if (InARFrame.smoothedSceneDepth)
		{
			SceneDepth = InARFrame.smoothedSceneDepth;
		}
		else
		{
			SceneDepth = InARFrame.sceneDepth;
		}
		
		[SceneDepth retain];
	}
#endif
}

template<class T>
static void ReleaseAndClear(T& Ref)
{
	if (Ref)
	{
		CFRelease(Ref);
		Ref = nullptr;
	}
}

void FAppleARKitFrame::ReleaseResources()
{
	ReleaseAndClear(CapturedYImage);
	ReleaseAndClear(CapturedCbCrImage);
	ReleaseAndClear(CameraImage);
	ReleaseAndClear(CameraDepth);
	
	[NativeFrame release];
	NativeFrame = nullptr;

#if SUPPORTS_ARKIT_3_0
	ReleaseAndClear(SegmentationBuffer);
	ReleaseAndClear(EstimatedDepthData);
#endif
	
#if SUPPORTS_ARKIT_4_0
	if (FAppleARKitAvailability::SupportsARKit40())
	{
		[SceneDepth release];
		SceneDepth = nullptr;
	}
#endif
}

FAppleARKitFrame::~FAppleARKitFrame()
{
	ReleaseResources();
}
#endif // SUPPORTS_ARKIT_1_0

FAppleARKitFrame& FAppleARKitFrame::operator=( const FAppleARKitFrame& Other )
{
	if (&Other == this)
	{
		return *this;
	}

#if SUPPORTS_ARKIT_4_0
	ReleaseResources();

	NativeFrame = Other.NativeFrame;
	[NativeFrame retain];
#endif // SUPPORTS_ARKIT_1_0

	// Member-wise copy
	Timestamp = Other.Timestamp;
	CapturedYImageSize = Other.CapturedYImageSize;
	CapturedCbCrImageSize = Other.CapturedCbCrImageSize;
	Camera = Other.Camera;
	LightEstimate = Other.LightEstimate;
	WorldMappingState = Other.WorldMappingState;

	Tracked2DPose = Other.Tracked2DPose;

	return *this;
}
