// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// AppleARKit
#include "AppleARKitAvailability.h"
#include "AppleARKitCamera.h"
#include "AppleARKitLightEstimate.h"

#include "AppleARKitFrame.generated.h"

/**
 * An object representing a frame processed by FAppleARKitSystem.
 * @discussion Each frame contains information about the current state of the scene.
 */
USTRUCT( Category="AppleARKit" )
struct APPLEARKIT_API FAppleARKitFrame
{
	GENERATED_BODY()
	
	FAppleARKitFrame() = default;
	FAppleARKitFrame(const FAppleARKitFrame& Other) = delete;

	/** 
	 * Assignment operator. CapturedImage is skipped as we don't need / want to retain access to the image buffer.
	 */
	FAppleARKitFrame& operator=(const FAppleARKitFrame& Other);

#if SUPPORTS_ARKIT_1_0
	/** 
	 * This is a conversion copy-constructor that takes a raw ARFrame and fills this structs members
	 * with the UE-ified versions of ARFrames properties.
	 *
	 * @param InARFrame 		- The frame to convert / copy
	 * @param MinCameraUV 		- The minimum (top left) UV used to render the passthrough camera
	 * @param MaxCameraUV 		- The maximum (bottom right) UV used to render the passthrough camera
	 */
	FAppleARKitFrame(ARFrame* InARFrame, const FVector2D MinCameraUV, const FVector2D MaxCameraUV, CVMetalTextureCacheRef MetalTextureCache);
	
	/** Destructor. Calls CFRelease on CapturedImage */
	virtual ~FAppleARKitFrame();

	void ReleaseResources();
#endif
	
	/** A timestamp identifying the frame. */
	double Timestamp = 0.0;

#if SUPPORTS_ARKIT_1_0

	/** The raw camera buffer from ARKit */
	CVPixelBufferRef CameraImage = nullptr;
	/** The raw camera depth info from ARKit (needs iPhone X) */
	AVDepthData* CameraDepth = nullptr;
	ARFrame* NativeFrame = nullptr;
	
	/** The frame's captured images */
    CVMetalTextureRef CapturedYImage = nullptr;
    CVMetalTextureRef CapturedCbCrImage = nullptr;
	
#endif
	
	/** The size in pixels of the frame's captured images. */
	FIntPoint CapturedYImageSize = FIntPoint::ZeroValue;
    FIntPoint CapturedCbCrImageSize = FIntPoint::ZeroValue;
	
	/** The camera used to capture the frame's image. */
	FAppleARKitCamera Camera;

	/**
	 * A light estimate representing the estimated light in the scene.
	 */
	FAppleARKitLightEstimate LightEstimate;
	
	/** The current world mapping state is reported on the frame */
	EARWorldMappingState WorldMappingState = EARWorldMappingState::NotAvailable;
	
	/** The current tracked 2D pose */
	FARPose2D Tracked2DPose;

#if SUPPORTS_ARKIT_3_0
	/** The person segmentation buffer from ARKit */
	CVPixelBufferRef SegmentationBuffer = nullptr;
		
	/** The estimated depth buffer for person segmentation from ARKit */
	CVPixelBufferRef EstimatedDepthData = nullptr;
#endif
	
#if SUPPORTS_ARKIT_4_0
	ARDepthData* SceneDepth = nullptr;
#endif

	/* 
	 * When adding new member variables, don't forget to handle them in the copy constructor and
	 * assignment operator above.
	 */
};
