// Copyright Epic Games, Inc. All Rights Reserved.

// AppleARKit
#include "AppleARKitLightEstimate.h"


#if SUPPORTS_ARKIT_1_0

FAppleARKitLightEstimate::FAppleARKitLightEstimate( ARLightEstimate* InARLightEstimate )
: bIsValid( InARLightEstimate != nullptr )
, AmbientIntensity( InARLightEstimate != nullptr ? InARLightEstimate.ambientIntensity : 0.0f )
, AmbientColorTemperatureKelvin( InARLightEstimate != nullptr ? InARLightEstimate.ambientColorTemperature : 0.0f )
{

}

#endif
