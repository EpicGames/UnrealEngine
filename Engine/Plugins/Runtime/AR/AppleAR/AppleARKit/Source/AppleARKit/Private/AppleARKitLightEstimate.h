// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE
#include "CoreMinimal.h"
#include "Math/SHMath.h"
#include "AppleARKitAvailability.h"

// ARKit
#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

// AppleARKit
#include "AppleARKitLightEstimate.generated.h"

/**
 * A light estimate represented as spherical harmonics
 */
USTRUCT( )
struct APPLEARKIT_API FAppleARKitLightEstimate
{
	GENERATED_BODY()

	// Default constructor
	FAppleARKitLightEstimate()
		: bIsValid(false)
		, AmbientIntensity(0.f)
		, AmbientColorTemperatureKelvin(0.f)
	{};

#if SUPPORTS_ARKIT_1_0

	/** 
	 * This is a conversion copy-constructor that takes a raw ARLightEstimate and fills this structs 
	 * members with the UE-ified versions of ARLightEstimate's properties.
	 */ 
	FAppleARKitLightEstimate( ARLightEstimate* InARLightEstimate );

#endif

	/** True if light estimation was enabled for the session and light estimation was successful */
	UPROPERTY()
	bool bIsValid;

	/**
	 * Ambient intensity of the lighting.
	 * 
	 * In a well lit environment, this value is close to 1000. It typically ranges from 0 
	 * (very dark) to around 2000 (very bright).
	 */
	UPROPERTY()
	float AmbientIntensity;
	
	/**
	 * Color Temperature in Kelvin of light
	 *
	 */
	UPROPERTY()
	float AmbientColorTemperatureKelvin;
};
