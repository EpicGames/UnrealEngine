// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitMeshData.h"
#include "AppleARKitConversion.h"
#include "AppleARKitSystem.h"
#include "AppleARKitModule.h"


#define USE_ARKIT_NORMALS 0

DECLARE_CYCLE_STAT(TEXT("Update Mesh Data"), STAT_UpdateMeshData, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("Get Classification"), STAT_GetClassification, STATGROUP_ARKIT);

TMap<FGuid, FARKitMeshData::MeshDataPtr> FARKitMeshData::CachedMeshData;
FCriticalSection FARKitMeshData::GlobalLock;

FARKitMeshData::~FARKitMeshData()
{	
#if SUPPORTS_ARKIT_3_5
	FScopeLock ScopeLock(&PendingMeshLock);
	if (FAppleARKitAvailability::SupportsARKit35() && PendingMeshGeometry)
	{
		[PendingMeshGeometry release];
		PendingMeshGeometry = nullptr;
	}
#endif
}

void FARKitMeshData::ClearAllMeshData()
{
	FScopeLock ScopeLock(&GlobalLock);
	CachedMeshData = {};
}

void FARKitMeshData::RemoveMeshData(const FGuid& InGuid)
{
	FScopeLock ScopeLock(&GlobalLock);
	CachedMeshData.Remove(InGuid);
}

void FARKitMeshData::UpdateMRMesh(const FTransform& MeshTransform, MeshDataPtr MeshData, UMRMeshComponent* InMeshComponent)
{
	if (!InMeshComponent)
	{
		return;
	}
	
#if SUPPORTS_ARKIT_3_5
	MeshData->UpdateMeshData();
#endif
	InMeshComponent->SetRelativeTransform(MeshTransform);
	
	((IMRMesh*)InMeshComponent)->SendBrickData({
		MeshData,
		0,
		MeshData->Vertices,
		MeshData->DummyUVData,
		MeshData->TangentData,
		MeshData->DummyColorData,
		MeshData->Indices,
		MeshData->BoundingBox.TransformBy(MeshTransform)
	});
}

FARKitMeshData::MeshDataPtr FARKitMeshData::GetMeshData(const FGuid& InGuid)
{
	if (auto Record = CachedMeshData.Find(InGuid))
	{
		return *Record;
	}
	
	return nullptr;
}

bool FARKitMeshData::GetClassificationAtLocation(const FVector& InWorldLocation, const FTransform& InLocalToWorldTransform, uint8& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff)
{
	SCOPE_CYCLE_COUNTER(STAT_GetClassification);
	
	if (!Classifications.Num())
	{
		return false;
	}
	
	const auto LocalLocation = InLocalToWorldTransform.InverseTransformPosition(InWorldLocation);
	if (!BoundingBox.IsInside(LocalLocation))
	{
		return false;
	}
	
	check(Classifications.Num() * 3 == Indices.Num());
	const auto DistanceSquared = MaxLocationDiff * MaxLocationDiff;
	
	for (auto FaceIndex = 0; FaceIndex < Classifications.Num(); ++FaceIndex)
	{
		const auto& Vert0 = Vertices[Indices[3 * FaceIndex]];
		const auto& Vert1 = Vertices[Indices[3 * FaceIndex + 1]];
		const auto& Vert2 = Vertices[Indices[3 * FaceIndex + 2]];
		const auto Center = (Vert0 + Vert1 + Vert2) / 3.f;
		if (((FVector)Center - LocalLocation).SizeSquared() < DistanceSquared) // distance less than 5cm
		{
			OutClassification = Classifications[FaceIndex];
			OutClassificationLocation = InLocalToWorldTransform.TransformPosition((FVector)Center);
			return true;
		}
	}
	
	return false;
}

#if SUPPORTS_ARKIT_3_5
TSharedPtr<FARKitMeshData, ESPMode::ThreadSafe> FARKitMeshData::CacheMeshData(const FGuid& InGuid, ARMeshAnchor* InMeshAnchor)
{
	if (!FAppleARKitAvailability::SupportsARKit35() || !InMeshAnchor)
	{
		return nullptr;
	}
	
	FScopeLock ScopeLock(&GlobalLock);
	if (!CachedMeshData.Contains(InGuid))
	{
		CachedMeshData.Add(InGuid, MakeShared<FARKitMeshData, ESPMode::ThreadSafe>());
	}
	
	MeshDataPtr MeshData = CachedMeshData[InGuid];
	MeshData->UpdateMeshData(InMeshAnchor);
	return MeshData;
}

FORCEINLINE void MergeBoxWith(FBox& Box, const FVector& Location)
{
	if (!Box.IsValid)
	{
		Box.Min = Box.Max = Location;
		Box.IsValid = true;
	}
	else
	{
		Box.Min = Box.Min.ComponentMin(Location);
		Box.Max = Box.Max.ComponentMax(Location);
	}
}

void FARKitMeshData::UpdateMeshData()
{
	if (!FAppleARKitAvailability::SupportsARKit35())
	{
		return;
	}
	
	SCOPE_CYCLE_COUNTER(STAT_UpdateMeshData);
	
	// Make sure the internal mesh data is only updated in a single thread as they're not protected!
	check(IsInGameThread());
	
	ARMeshGeometry* GeometryCopy = nullptr;
	{
		FScopeLock ScopeLock(&PendingMeshLock);
		GeometryCopy = PendingMeshGeometry;
		PendingMeshGeometry = nullptr;
	}
	
	// Make sure at least the basic data set is there
	if (!GeometryCopy)
	{
		return;
	}
	
	BoundingBox.Init();
	
	// https://developer.apple.com/documentation/arkit/armeshanchor?language=objc
	auto bHasInvalidData = true;
	do
	{
		// Copy the vertices
		{
			ARGeometrySource* InVertices = GeometryCopy.vertices;
			const auto NumVertices = InVertices.count;
			if (InVertices.componentsPerVector == 3 &&
				InVertices.format == MTLVertexFormatFloat3 &&
				InVertices.stride == sizeof(FVector) &&
				InVertices.buffer &&
				InVertices.buffer.contents)
			{
				Vertices.SetNumUninitialized(NumVertices);
				const float* SourceVertices = (const float*)InVertices.buffer.contents;
				for (auto Index = 0; Index < NumVertices; ++Index, SourceVertices += 3)
				{
					// See FAppleARKitConversion::ToFVector
					Vertices[Index] = FVector3f(-SourceVertices[2], SourceVertices[0], SourceVertices[1]) * FAppleARKitConversion::ToUEScale();
					MergeBoxWith(BoundingBox, static_cast<FVector>(Vertices[Index]));
				}
			}
			else
			{
				UE_LOGF(LogAppleARKit, Error, "Mesh anchor contains incorrectly formatted vertex data!");
				break;
			}
		}
		
		// Copy the indices
		{
			ARGeometryElement* InFaces = GeometryCopy.faces;
			if (InFaces.indexCountPerPrimitive == 3 &&
				InFaces.bytesPerIndex == sizeof(uint32) &&
				InFaces.primitiveType == ARGeometryPrimitiveTypeTriangle &&
				InFaces.buffer &&
				InFaces.buffer.contents)
			{
				const auto NumIndices = InFaces.count * 3;
				Indices.SetNumUninitialized(NumIndices);
				FMemory::Memcpy(Indices.GetData(), InFaces.buffer.contents, sizeof(uint32) * NumIndices);
			}
			else
			{
				UE_LOGF(LogAppleARKit, Error, "Mesh anchor contains incorrectly formatted index data!");
				break;
			}
		}
		
		// Calculate the tangents and normals
#if USE_ARKIT_NORMALS
		{
			// This code is not used, it's merely here as a reference of what the normal data structure looks like.
			// We're not using it because the normal data from ARKit is very wonky
			ARGeometrySource* InNormals = GeometryCopy.normals;
			const auto NumVertices = InNormals.count;
			if (InNormals.componentsPerVector == 3 &&
				InNormals.format == MTLVertexFormatFloat3 &&
				InNormals.stride == sizeof(FVector) &&
				InNormals.buffer &&
				InNormals.buffer.contents)
			{
				TangentData.SetNumUninitialized(NumVertices * 2);
				const float* SourceNormals = (const float*)InNormals.buffer.contents;
				const auto MeshCenter = BoundingBox.GetCenter();
				for (auto Index = 0; Index < NumVertices; ++Index, SourceNormals += 3)
				{
					// See FAppleARKitConversion::ToFVector
					const FVector Normal(-SourceNormals[2], SourceNormals[0], SourceNormals[1]);
					// We don't have UV info to calculate the correct tangent here
					// so juse use an arbitrary vector to get a tangent perpendicular to the normal...
					const auto Cross = (MeshCenter - Vertices[Index]) ^ Normal;
					auto Tangent = (Normal ^ Cross);
					Tangent.Normalize();
					TangentData[2 * Index + 0] = FPackedNormal(Tangent);
					TangentData[2 * Index + 1] = FPackedNormal(Normal);
				}
			}
			else
			{
				UE_LOGF(LogAppleARKit, Error, "Mesh anchor contains incorrectly formatted normal data!");
				break;
			}
		}
#else // USE_ARKIT_NORMALS
		{
			// TODO: Revisit LWC and AR geometry; temp fix to unbreak build.
			TArray<FVector> LwcVertices;
			LwcVertices.SetNumUninitialized(Vertices.Num());
			for (int32 i = 0; i < Vertices.Num(); ++i)
			{
				LwcVertices[i] = FVector(Vertices[i].X, Vertices[i].Y, Vertices[i].Z);
			}

			FAccumulatedNormal::CalculateVertexNormals(AccumulatedNormals, LwcVertices, Indices, TangentData, BoundingBox.GetCenter(), 100.f);
		}
#endif // USE_ARKIT_NORMALS
		
		// Copy the classification
		{
			// https://developer.apple.com/documentation/arkit/armeshclassification?language=objc
			if (ARGeometrySource* InClassification = GeometryCopy.classification)
			{
				const auto NumFaces = InClassification.count;
				if (InClassification.componentsPerVector == 1 &&
					InClassification.format == MTLVertexFormatUChar &&
					InClassification.stride == sizeof(uint8) &&
					InClassification.buffer &&
					InClassification.buffer.contents)
				{
					Classifications.SetNumUninitialized(NumFaces);
					FMemory::Memcpy(Classifications.GetData(), InClassification.buffer.contents, sizeof(uint8) * NumFaces);
				}
				else
				{
					UE_LOGF(LogAppleARKit, Error, "Mesh anchor contains incorrectly formatted classification data!");
					break;
				}
			}
		}
		
		DummyUVData.SetNumZeroed(Vertices.Num());
		DummyColorData.SetNumZeroed(Vertices.Num());
		bHasInvalidData = false;
	}
	while(false);
	
	if (bHasInvalidData)
	{
		ClearData();
	}
	
	[GeometryCopy release];
}

void FARKitMeshData::ClearData()
{
	Vertices = {};
	TangentData = {};
	Indices = {};
	Classifications = {};
	DummyUVData = {};
	DummyColorData = {};
}

void FARKitMeshData::UpdateMeshData(ARMeshAnchor* InMeshAnchor)
{
	if (!FAppleARKitAvailability::SupportsARKit35() || !InMeshAnchor)
	{
		return;
	}
	
	FScopeLock ScopeLock(&PendingMeshLock);
	
	if (PendingMeshGeometry)
	{
		[PendingMeshGeometry release];
		PendingMeshGeometry = nullptr;
	}
	
	PendingMeshGeometry = InMeshAnchor.geometry;
	[PendingMeshGeometry retain];
}
#endif
