// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitModule.h"
#include "AppleARKitSystem.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "CoreGlobals.h"
#include "Misc/CoreDelegates.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	#include "AppleARKitSettings.h"
#endif

#define LOCTEXT_NAMESPACE "ARKit"

TWeakPtr<class FAppleARKitSystem, ESPMode::ThreadSafe> FAppleARKitARKitSystemPtr;

TSharedPtr<class IXRTrackingSystem, ESPMode::ThreadSafe> FAppleARKitModule::CreateTrackingSystem()
{
#if PLATFORM_IOS
	auto NewARKitSystem = AppleARKitSupport::CreateAppleARKitSystem();
	NewARKitSystem->GetARCompositionComponent()->InitializeARSystem();
	FAppleARKitARKitSystemPtr = NewARKitSystem;
	FGenericCrashContext::SetEngineData(TEXT("XR"), TEXT("XR AppleARKit"));
    return NewARKitSystem;
#else
	return TSharedPtr<class IXRTrackingSystem, ESPMode::ThreadSafe>();
#endif
}

TSharedPtr<class FAppleARKitSystem, ESPMode::ThreadSafe> FAppleARKitModule::GetARKitSystem()
{
    return FAppleARKitARKitSystemPtr.Pin();
}

FString FAppleARKitModule::GetModuleKeyName() const
{
    static const FString ModuleKeyName(TEXT("AppleARKit"));
    return ModuleKeyName;
}

void FAppleARKitModule::StartupModule()
{
	ensureMsgf(FModuleManager::Get().LoadModule("AugmentedReality"), TEXT("ARKit depends on the AugmentedReality module."));
	IHeadMountedDisplayModule::StartupModule();

	FCoreDelegates::OnPreExit.AddRaw(this, &FAppleARKitModule::PreExit);

#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAppleARKitModule::PostEngineInit);
#endif
	
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("AppleARKit"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/AppleARKit"), PluginShaderDir);
}

void FAppleARKitModule::PreExit()
{
#if WITH_EDITOR
	UnregisterSettings();
#endif
	if (FAppleARKitARKitSystemPtr.IsValid())
	{
		FAppleARKitARKitSystemPtr.Pin()->Shutdown();
	}
	FAppleARKitARKitSystemPtr.Reset();
}

void FAppleARKitModule::ShutdownModule()
{
	IHeadMountedDisplayModule::ShutdownModule();
}

#if WITH_EDITOR
void FAppleARKitModule::PostEngineInit()
{
	RegisterSettings();
}

void FAppleARKitModule::RegisterSettings()
{
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "Apple ARKit",
				LOCTEXT("ARKitSettingsName", "Apple ARKit"),
				LOCTEXT("ARKitSettingsDescription", "Configure the Apple ARKit plug-in."),
				GetMutableDefault<UAppleARKitSettings>()
			);
		}
	}
}

void FAppleARKitModule::UnregisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Apple ARKit");
	}
}
#endif


IMPLEMENT_MODULE(FAppleARKitModule, AppleARKit);

DEFINE_LOG_CATEGORY(LogAppleARKit);

#undef LOCTEXT_NAMESPACE
