// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "IHeadMountedDisplayModule.h"
#include "Features/IModularFeature.h"
#include "AppleARKitSystem.h"

class APPLEARKIT_API FAppleARKitModule : public IHeadMountedDisplayModule
{
public:
	virtual TSharedPtr<class IXRTrackingSystem, ESPMode::ThreadSafe> CreateTrackingSystem() override;
	
	static TSharedPtr<class FAppleARKitSystem, ESPMode::ThreadSafe> GetARKitSystem();
	
	FString GetModuleKeyName() const override;

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

    void PreExit();

#if WITH_EDITOR
	void PostEngineInit();
    void RegisterSettings();
    void UnregisterSettings();
#endif
};

DECLARE_LOG_CATEGORY_EXTERN(LogAppleARKit, Log, All);

