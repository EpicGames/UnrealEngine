// Copyright Epic Games, Inc. All Rights Reserved.

// AppleARKit
#include "AppleARKitSessionDelegate.h"
#include "AppleARKitSystem.h"
#include "AppleARKitModule.h"
#include "AppleARKitFrame.h"
#include "RenderingThread.h"

#if SUPPORTS_ARKIT_1_0

@implementation FAppleARKitSessionDelegate
{
	FAppleARKitSystem* _AppleARKitSystem;
	TArray<FVector2D> PassthroughCameraUVs;
	FCriticalSection CameraUVsLock;

	CVMetalTextureCacheRef _metalTextureCache;
}



- (id)initWithAppleARKitSystem:(FAppleARKitSystem*)InAppleARKitSystem
{
	self = [super init];
	if (self)
	{
		UE_LOGF(LogAppleARKit, Log, "Delegate created with session: %p", InAppleARKitSystem);
		_AppleARKitSystem = InAppleARKitSystem;
		_metalTextureCache = NULL;
	}
	return self;
}

- (void)setMetalTextureCache:(CVMetalTextureCacheRef)InMetalTextureCache
{
	// Release current?
	if ( _metalTextureCache != nullptr )
	{
		CFRelease( _metalTextureCache );
	}
	// Set new & retain
	_metalTextureCache = InMetalTextureCache;
	if ( _metalTextureCache != nullptr )
	{
		CFRetain( _metalTextureCache );
	}
}
#pragma mark - ARSessionDelegate Methods

- (void)session:(ARSession *)session didUpdateFrame:(ARFrame *)frame 
{
	if (!_metalTextureCache)
	{
		UE_LOGF(LogAppleARKit, Log, "Delegate didUpdateFrame with no valid _metalTextureCache (ignoring)");
		return;
	}
	
	// Update the camera UVs
	TSharedPtr<IXRCamera, ESPMode::ThreadSafe> Camera = _AppleARKitSystem->GetXRCamera(0);
	if (Camera)
	{
		ENQUEUE_RENDER_COMMAND(UpdateCameraUVsCommand)([self, Camera](FRHICommandListImmediate& RHICmdList)
		{
			FScopeLock ScopeLock(&CameraUVsLock);
			Camera->GetPassthroughCameraUVs_RenderThread(PassthroughCameraUVs);
		});
	}
	
	FVector2D MinCameraUV(0.f, 0.f);
	FVector2D MaxCameraUV(1.f, 1.f);
	{
		FScopeLock ScopeLock(&CameraUVsLock);
		if (PassthroughCameraUVs.Num() == 4)
		{
			MinCameraUV = PassthroughCameraUVs[0];
			MaxCameraUV = PassthroughCameraUVs[3];
		}
	}
	
	// Bundle results into FAppleARKitFrame
	TSharedPtr< FAppleARKitFrame, ESPMode::ThreadSafe > AppleARKitFrame( new FAppleARKitFrame( frame, MinCameraUV, MaxCameraUV, _metalTextureCache ) );
	
	// Pass result to session
	_AppleARKitSystem->SessionDidUpdateFrame_DelegateThread( AppleARKitFrame );
}

- (void)session:(ARSession *)session didFailWithError:(NSError *)error
{
	// Log error
	NSString *ErrorDescription = [error localizedDescription];
	UE_LOGF( LogAppleARKit, Warning, "Session failed with error: %ls", *FString(ErrorDescription) );
}

- (void)session:(ARSession *)session didAddAnchors:(NSArray<ARAnchor*>*)anchors 
{
	// Pass updates to sessiom
	_AppleARKitSystem->SessionDidAddAnchors_DelegateThread( anchors );
}

- (void)session:(ARSession *)session didUpdateAnchors:(NSArray<ARAnchor*>*)anchors 
{    
	// Pass updates to sessiom
	_AppleARKitSystem->SessionDidUpdateAnchors_DelegateThread( anchors );
}

- (void)session:(ARSession *)session didRemoveAnchors:(NSArray<ARAnchor*>*)anchors 
{    
	// Pass updates to sessiom
	_AppleARKitSystem->SessionDidRemoveAnchors_DelegateThread( anchors );
}

@end
#endif

