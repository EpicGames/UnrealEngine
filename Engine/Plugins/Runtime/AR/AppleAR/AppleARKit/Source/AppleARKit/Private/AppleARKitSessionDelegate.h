// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "AppleARKitAvailability.h"

#if SUPPORTS_ARKIT_1_0

// ARKit
#include <ARKit/ARKit.h>

@interface FAppleARKitSessionDelegate : NSObject<ARSessionDelegate>
{
}

- (id)initWithAppleARKitSystem:(class FAppleARKitSystem*)InAppleARKitSystem;

- (void)setMetalTextureCache:(CVMetalTextureCacheRef)InMetalTextureCache;

@end

#endif
