// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitSystem.h"
#include "DefaultXRCamera.h"
#include "AppleARKitSessionDelegate.h"
#include "Misc/ScopeLock.h"
#include "AppleARKitModule.h"
#include "AppleARKitConversion.h"
#include "AppleARKitVideoOverlay.h"
#include "AppleARKitFrame.h"
#include "AppleARKitConversion.h"
#include "GeneralProjectSettings.h"
#include "ARSessionConfig.h"
#include "ARLifeCycleComponent.h"
#include "AppleARKitSettings.h"
#include "AppleARKitTrackable.h"
#include "ARLightEstimate.h"
#include "ARTraceResult.h"
#include "ARPin.h"
#include "ARKitTrackables.h"
#include "Async/Async.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/IConsoleManager.h"
#include "Misc/FileHelper.h"
#include "ARBlueprintLibrary.h"
#include "ARKitGeoTrackingSupport.h"
#include "RenderGraphBuilder.h"
#include "DataDrivenShaderPlatformInfo.h"

#if PLATFORM_APPLE
#include "IMetalDynamicRHI.h"
#endif

// For mesh occlusion
#include "MRMeshComponent.h"
#include "AROriginActor.h"

// To separate out the face ar library linkage from standard ar apps
#include "AppleARKitFaceSupport.h"
#include "AppleARKitPoseTrackingLiveLink.h"

// For orientation changed
#include "Misc/CoreDelegates.h"

#if PLATFORM_IOS
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Wpartial-availability"
#endif

// This flag might belong in UARSessionConfig, if Apple couldn't fix the bug where
// ARKit stops tracking surfaces after the back camera gets used by some other code
static TAutoConsoleVariable<int32> CVarReleaseSessionWhenStopped(
    TEXT("ar.ARKit.ReleaseSessionWhenStopped"),
    0,
	TEXT("Whether to release the ARKit session object when the AR session is stopped."));

static float SceneDepthBufferSizeScale = 1.f;
static FAutoConsoleVariableRef CVarSceneDepthBufferSizeScale(
	TEXT("arkit.SceneDepthBufferSizeScale"),
    SceneDepthBufferSizeScale,
	TEXT("If > 1, the scene depth buffer and confidence map will be upscaled to have the sizes multiplied by this value.")
	);

static float SceneDepthBufferBlurAmount = 0.f;
static FAutoConsoleVariableRef CVarSceneDepthBufferBlurAmount(
	TEXT("arkit.SceneDepthBufferBlurAmount"),
    SceneDepthBufferBlurAmount,
	TEXT("If > 0, the scene depth buffer and confidence map will be applied a gaussian blur whose sigma is this value.")
	);

DECLARE_CYCLE_STAT(TEXT("SessionDidUpdateFrame_DelegateThread"), STAT_FAppleARKitSystem_SessionUpdateFrame, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("SessionDidAddAnchors_DelegateThread"), STAT_FAppleARKitSystem_SessionDidAddAnchors, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("SessionDidUpdateAnchors_DelegateThread"), STAT_FAppleARKitSystem_SessionDidUpdateAnchors, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("SessionDidRemoveAnchors_DelegateThread"), STAT_FAppleARKitSystem_SessionDidRemoveAnchors, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("UpdateARKitPerf"), STAT_FAppleARKitSystem_UpdateARKitPerf, STATGROUP_ARKIT);
DECLARE_DWORD_COUNTER_STAT(TEXT("ARKit CPU %"), STAT_ARKitThreads, STATGROUP_ARKIT);

DECLARE_FLOAT_COUNTER_STAT(TEXT("ARKit Frame to Delegate Delay (ms)"), STAT_ARKitFrameToDelegateDelay, STATGROUP_ARKIT);

// Copied from IOSPlatformProcess because it's not accessible by external code
#define GAME_THREAD_PRIORITY 47
#define RENDER_THREAD_PRIORITY 45

#if PLATFORM_IOS && !PLATFORM_TVOS
// Copied from IOSPlatformProcess because it's not accessible by external code
static void SetThreadPriority(int32 Priority)
{
	struct sched_param Sched;
	FMemory::Memzero(&Sched, sizeof(struct sched_param));
	
	// Read the current priority and policy
	int32 CurrentPolicy = SCHED_RR;
	pthread_getschedparam(pthread_self(), &CurrentPolicy, &Sched);
	
	// Set the new priority and policy (apple recommended FIFO for the two main non-working threads)
	int32 Policy = SCHED_FIFO;
	Sched.sched_priority = Priority;
	pthread_setschedparam(pthread_self(), Policy, &Sched);
}
#else
static void SetThreadPriority(int32 Priority)
{
	// Ignored
}
#endif

//
//  FAppleARKitXRCamera
//

class FAppleARKitXRCamera : public FDefaultXRCamera
{
public:
	FAppleARKitXRCamera(const FAutoRegister& AutoRegister, FAppleARKitSystem& InTrackingSystem, int32 InDeviceId)
	: FDefaultXRCamera( AutoRegister, &InTrackingSystem, InDeviceId )
	, ARKitSystem( InTrackingSystem )
	{}
	
	void AdjustThreadPriority(int32 NewPriority)
	{
		ThreadPriority.Set(NewPriority);
	}
	
	FAppleARKitVideoOverlay& GetOverlay() { return VideoOverlay; }
	
private:
	//~ FDefaultXRCamera
	void OverrideFOV(float& InOutFOV)
	{
		// @todo arkit : is it safe not to lock here? Theoretically this should only be called on the game thread.
		ensure(IsInGameThread());
		const bool bShouldOverrideFOV = ARKitSystem.GetARCompositionComponent()->GetSessionConfig().ShouldRenderCameraOverlay();
		if (bShouldOverrideFOV && ARKitSystem.GameThreadFrame.IsValid())
		{
			if (ARKitSystem.DeviceOrientation == EDeviceScreenOrientation::Portrait || ARKitSystem.DeviceOrientation == EDeviceScreenOrientation::PortraitUpsideDown)
			{
				// Portrait
				InOutFOV = ARKitSystem.GameThreadFrame->Camera.GetVerticalFieldOfViewForScreen(EAppleARKitBackgroundFitMode::Fill);
			}
			else
			{
				// Landscape
				InOutFOV = ARKitSystem.GameThreadFrame->Camera.GetHorizontalFieldOfViewForScreen(EAppleARKitBackgroundFitMode::Fill);
			}
		}
	}
	
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override
	{
		FDefaultXRCamera::SetupView(InViewFamily, InView);
	}
	
	virtual void SetupViewProjectionMatrix(FSceneViewProjectionData& InOutProjectionData) override
	{
		if (ARKitSystem.GameThreadFrame.IsValid()) 
		{
			ARKitSystem.GameThreadFrame->Camera.GetViewProjectionMatrix(ARKitSystem.DeviceOrientation, InOutProjectionData);
		}
	}
	
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override
	{
		FDefaultXRCamera::BeginRenderViewFamily(InViewFamily);
	}
	
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override
	{
		// Adjust our thread priority if requested
		if (LastThreadPriority.GetValue() != ThreadPriority.GetValue())
		{
			SetThreadPriority(ThreadPriority.GetValue());
			LastThreadPriority.Set(ThreadPriority.GetValue());
		}
		FDefaultXRCamera::PreRenderView_RenderThread(GraphBuilder, InView);
	}

	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override
	{
		// Grab the latest frame from ARKit
		{
			FScopeLock ScopeLock(&ARKitSystem.FrameLock);
			ARKitSystem.RenderThreadFrame = ARKitSystem.LastReceivedFrame;
		}

		FDefaultXRCamera::PreRenderViewFamily_RenderThread(GraphBuilder, InViewFamily);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FPostBasePassViewExtensionParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
	
	virtual void PostRenderBasePassDeferred_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView, const FRenderTargetBindingSlots& RenderTargets, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures) override
	{
		if (ARKitSystem.RenderThreadFrame.IsValid())
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FPostBasePassViewExtensionParameters>();
			PassParameters->RenderTargets = RenderTargets;
			PassParameters->SceneTextures = SceneTextures;

			GraphBuilder.AddPass(RDG_EVENT_NAME("RenderVideoOverlay_RenderThread"), PassParameters, ERDGPassFlags::Raster, [this, &InView](FRHICommandListImmediate& RHICmdList)
			{
				VideoOverlay.RenderVideoOverlay_RenderThread(RHICmdList, InView, *ARKitSystem.RenderThreadFrame, ARKitSystem.GetWorldToMetersScale());
			});
		}
	}

	virtual void PostRenderBasePassMobile_RenderThread(FRHICommandList& RHICmdList, FSceneView& InView) override
	{
		if (ARKitSystem.RenderThreadFrame.IsValid())
		{
			VideoOverlay.RenderVideoOverlay_RenderThread(RHICmdList, InView, *ARKitSystem.RenderThreadFrame, ARKitSystem.GetWorldToMetersScale());
		}
	}
	
	virtual bool GetPassthroughCameraUVs_RenderThread(TArray<FVector2D>& OutUVs) override
	{
		return VideoOverlay.GetPassthroughCameraUVs_RenderThread(OutUVs, ARKitSystem.DeviceOrientation);
	}

	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override
	{
		// Base implementation needs this call as it updates bCurrentFrameIsStereoRendering as a side effect.
		// We'll ignore the result however.
		FDefaultXRCamera::IsActiveThisFrame_Internal(Context);

		// Check to see if they have disabled the automatic rendering or not
		// Most Face AR apps that are driving other meshes using the face capture (animoji style) will disable this.
		const bool bRenderOverlay =
			ARKitSystem.OnGetARSessionStatus().Status == EARSessionStatus::Running &&
			ARKitSystem.GetARCompositionComponent()->GetSessionConfig().ShouldRenderCameraOverlay();

#if SUPPORTS_ARKIT_1_0
		if (FAppleARKitAvailability::SupportsARKit10())
		{
			return bRenderOverlay;
		}
#endif
		return false;
	}
	//~ FDefaultXRCamera
	
private:
	FAppleARKitSystem& ARKitSystem;
	FAppleARKitVideoOverlay VideoOverlay;
	
	// Thread priority support
	FThreadSafeCounter ThreadPriority;
	FThreadSafeCounter LastThreadPriority;
};

//
//  FAppleARKitSystem
//

FAppleARKitSystem::FAppleARKitSystem()
: FXRTrackingSystemBase(this)
, DeviceOrientation(EDeviceScreenOrientation::Unknown)
, DerivedTrackingToUnrealRotation(FRotator::ZeroRotator)
, LightEstimate(nullptr)
, CameraDepth(nullptr)
, LastTrackedGeometry_DebugId(0)
, PoseTrackingARLiveLink(nullptr)
, TimecodeProvider(nullptr)
{
	// See Initialize(), as we need access to SharedThis()
#if SUPPORTS_ARKIT_1_0
	IAppleImageUtilsPlugin::Load();
#endif

	SpawnARActorDelegateHandle = UARLifeCycleComponent::OnSpawnARActorDelegate.AddRaw(this, &FAppleARKitSystem::OnSpawnARActor);
	IModularFeatures::Get().RegisterModularFeature(UARKitGeoTrackingSupport::GetModularFeatureName(), GetMutableDefault<UARKitGeoTrackingSupport>());
}

FAppleARKitSystem::~FAppleARKitSystem()
{
	IModularFeatures::Get().UnregisterModularFeature(UARKitGeoTrackingSupport::GetModularFeatureName(), GetMutableDefault<UARKitGeoTrackingSupport>());
	UARLifeCycleComponent::OnSpawnARActorDelegate.Remove(SpawnARActorDelegateHandle);
}

void FAppleARKitSystem::Shutdown()
{
#if SUPPORTS_ARKIT_1_0
	if (Session != nullptr)
	{
		FaceARSupport = nullptr;
		PoseTrackingARLiveLink = nullptr;
		[Session pause];
		Session.delegate = nullptr;
		[Session release];
		Session = nullptr;
	}
#endif
	
#if SUPPORTS_ARKIT_3_0
	[CoachingOverlay release];
	CoachingOverlay = nullptr;
#endif
	
	CameraDepth = nullptr;
	CameraImage = nullptr;
	
	SceneDepthMap = nullptr;
	SceneDepthConfidenceMap = nullptr;
}

bool FAppleARKitSystem::IsARAvailable() const
{
#if SUPPORTS_ARKIT_1_0
	return FAppleARKitAvailability::SupportsARKit10();
#else
	return false;
#endif
}

template<class T>
static T* GetModularInterface()
{
	if (auto Implementation = IModularFeatures::Get().GetModularFeatureImplementation(T::GetModularFeatureName(), 0))
	{
		return static_cast<T*>(Implementation);
	}
	return nullptr;
}

void FAppleARKitSystem::CheckForFaceARSupport(UARSessionConfig* InSessionConfig)
{
	if (InSessionConfig->GetSessionType() != EARSessionType::Face)
	{
		// Clear the face ar support so we don't forward to it
		FaceARSupport = nullptr;
		return;
	}
	
	// We need to get the face support from the factory method, which is a modular feature to avoid dependencies
	FaceARSupport = GetModularInterface<IAppleARKitFaceSupport>();
	ensureAlwaysMsgf(FaceARSupport, TEXT("Face AR session has been requested but the face ar plugin is not enabled"));
}

void FAppleARKitSystem::CheckForPoseTrackingARLiveLink(UARSessionConfig* InSessionConfig)
{
#if SUPPORTS_ARKIT_3_0	
	if (InSessionConfig->GetSessionType() != EARSessionType::PoseTracking)
	{
		// Clear the face ar support so we don't forward to it
		PoseTrackingARLiveLink = nullptr;
		return;
	}
	
	PoseTrackingARLiveLink = GetModularInterface<IAppleARKitPoseTrackingLiveLink>();
	ensureAlwaysMsgf(PoseTrackingARLiveLink, TEXT("Body Tracking AR session has been requested but the body tracking ar plugin is not enabled"));
#endif
}

FName FAppleARKitSystem::GetSystemName() const
{
	static const FName AppleARKitSystemName(TEXT("AppleARKit"));
	return AppleARKitSystemName;
}

int32 FAppleARKitSystem::GetXRSystemFlags() const
{
	return EXRSystemFlags::IsAR | EXRSystemFlags::IsTablet;
}


bool FAppleARKitSystem::GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition)
{
	if (DeviceId == IXRTrackingSystem::HMDDeviceId && GameThreadFrame.IsValid() && IsHeadTrackingAllowed())
	{
		// Do not have to lock here, because we are on the game
		// thread and GameThreadFrame is only written to from the game thread.
		
		
		// Apply alignment transform if there is one.
		FTransform CurrentTransform(GameThreadFrame->Camera.Orientation, GameThreadFrame->Camera.Translation);
		CurrentTransform = FTransform(DerivedTrackingToUnrealRotation) * CurrentTransform;
		CurrentTransform *= GetARCompositionComponent()->GetAlignmentTransform();
		
		
		// Apply counter-rotation to compensate for mobile device orientation
		OutOrientation = CurrentTransform.GetRotation();
		OutPosition = CurrentTransform.GetLocation();

		return true;
	}
	else
	{
		return false;
	}
}

FString FAppleARKitSystem::GetVersionString() const
{
	return TEXT("AppleARKit - V1.0");
}


bool FAppleARKitSystem::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		static const int32 DeviceId = IXRTrackingSystem::HMDDeviceId;
		OutDevices.Add(DeviceId);
		return true;
	}
	return false;
}

void FAppleARKitSystem::CalcTrackingToWorldRotation()
{
	// We rotate the camera to counteract the portrait vs. landscape viewport rotation
	DerivedTrackingToUnrealRotation = FRotator::ZeroRotator;

	const EARWorldAlignment WorldAlignment = GetARCompositionComponent()->GetSessionConfig().GetWorldAlignment();
	if (WorldAlignment == EARWorldAlignment::Gravity || WorldAlignment == EARWorldAlignment::GravityAndHeading)
	{
		switch (DeviceOrientation)
		{
			case EDeviceScreenOrientation::Portrait:
				DerivedTrackingToUnrealRotation = FRotator(0.0f, 0.0f, -90.0f);
				break;
				
			case EDeviceScreenOrientation::PortraitUpsideDown:
				DerivedTrackingToUnrealRotation = FRotator(0.0f, 0.0f, 90.0f);
				break;
				
			default:
			case EDeviceScreenOrientation::LandscapeRight:
				break;
				
			case EDeviceScreenOrientation::LandscapeLeft:
				DerivedTrackingToUnrealRotation = FRotator(0.0f, 0.0f, 180.0f);
				break;
		}
	}
	// Camera aligned which means +X is to the right along the long axis
	else
	{
		switch (DeviceOrientation)
		{
			case EDeviceScreenOrientation::Portrait:
				DerivedTrackingToUnrealRotation = FRotator(0.0f, 0.0f, 90.0f);
				break;
				
			case EDeviceScreenOrientation::PortraitUpsideDown:
				DerivedTrackingToUnrealRotation = FRotator(0.0f, 0.0f, -90.0f);
				break;
				
			default:
			case EDeviceScreenOrientation::LandscapeLeft:
				DerivedTrackingToUnrealRotation = FRotator(0.0f, 0.0f, -180.0f);
				break;
				
			case EDeviceScreenOrientation::LandscapeRight:
				break;
		}
	}
}

#if PLATFORM_APPLE
static void UpdateCameraImageFromPixelBuffer(UAppleARKitTextureCameraImage*& CameraImage, CVPixelBufferRef Buffer, EARTextureType TextureType, double Timestamp, EPixelFormat PixelFormat, const CFStringRef ColorSpace = kCGColorSpaceGenericRGBLinear, FImageBlurParams BlurParam = {})
{
	if (!Buffer)
	{
		return;
	}
	
	if (!CameraImage)
	{
		CameraImage = UARTexture::CreateARTexture<UAppleARKitTextureCameraImage>(TextureType);
	}
	
	CameraImage->UpdateCameraImage(Timestamp, Buffer, PixelFormat, ColorSpace, BlurParam);
}
#endif

void FAppleARKitSystem::UpdateFrame()
{
	FScopeLock ScopeLock( &FrameLock );
	// This might get called multiple times per frame so only update if delegate version is newer
	if (!GameThreadFrame.IsValid() || !LastReceivedFrame.IsValid() ||
		GameThreadFrame->Timestamp < LastReceivedFrame->Timestamp)
	{
		GameThreadFrame = LastReceivedFrame;
		if (GameThreadFrame.IsValid())
		{
#if SUPPORTS_ARKIT_1_0
			if (GameThreadFrame->CameraDepth != nullptr)
			{
				// Reuse the UObjects because otherwise the time between GCs causes ARKit to be starved of resources
				CameraDepth->Init(FPlatformTime::Seconds(), GameThreadFrame->CameraDepth);
			}
#endif
			
#if SUPPORTS_ARKIT_4_0
			if (FAppleARKitAvailability::SupportsARKit40() && GameThreadFrame->SceneDepth)
			{
				FImageBlurParams BlurParam = { SceneDepthBufferBlurAmount, SceneDepthBufferSizeScale };
				UpdateCameraImageFromPixelBuffer(MutableView(SceneDepthMap), GameThreadFrame->SceneDepth.depthMap, EARTextureType::SceneDepthMap, GameThreadFrame->Timestamp, PF_R32_FLOAT, kCGColorSpaceGenericRGBLinear, BlurParam);
				
				// For some reason int8 texture needs to use the sRGB color space to ensure the values are intact after CIImage processing
				UpdateCameraImageFromPixelBuffer(MutableView(SceneDepthConfidenceMap), GameThreadFrame->SceneDepth.confidenceMap, EARTextureType::SceneDepthConfidenceMap, GameThreadFrame->Timestamp, PF_G8, kCGColorSpaceSRGB, BlurParam);
				
				if (FAppleARKitXRCamera* Camera = GetARKitXRCamera())
				{
					Camera->GetOverlay().UpdateSceneDepthTextures(SceneDepthMap, SceneDepthConfidenceMap);
				}
			}
#endif
		}
	}
}

void FAppleARKitSystem::ResetOrientationAndPosition(float Yaw)
{
	// @todo arkit implement FAppleARKitSystem::ResetOrientationAndPosition
}

bool FAppleARKitSystem::IsHeadTrackingAllowed() const
{
	// Check to see if they have disabled the automatic camera tracking or not
	// For face AR tracking movements of the device most likely won't want to be tracked
	const bool bEnableCameraTracking =
		OnGetARSessionStatus().Status == EARSessionStatus::Running &&
		GetARCompositionComponent()->GetSessionConfig().ShouldEnableCameraTracking();

#if SUPPORTS_ARKIT_1_0
	if (FAppleARKitAvailability::SupportsARKit10())
	{
		return bEnableCameraTracking;
	}
	else
	{
		return false;
	}
#else
	return false;
#endif
}

TSharedPtr<class IXRCamera, ESPMode::ThreadSafe> FAppleARKitSystem::GetXRCamera(int32 DeviceId)
{
	// Don't create/load UObjects on the render thread
	if (!XRCamera.IsValid() && IsInGameThread())
	{
		TSharedRef<FAppleARKitXRCamera, ESPMode::ThreadSafe> NewCamera = FSceneViewExtensions::NewExtension<FAppleARKitXRCamera>(*this, DeviceId);
		XRCamera = NewCamera;
	}
	
	return XRCamera;
}

FAppleARKitXRCamera* FAppleARKitSystem::GetARKitXRCamera()
{
	return (FAppleARKitXRCamera*)GetXRCamera(0).Get();
}

float FAppleARKitSystem::GetWorldToMetersScale() const
{
	// @todo arkit FAppleARKitSystem::GetWorldToMetersScale needs a real scale somehow
	return 100.0f;
}

void FAppleARKitSystem::OnBeginRendering_GameThread(FSceneViewFamily& InViewFamily)
{
	UpdateFrame();
}

bool FAppleARKitSystem::OnStartGameFrame(FWorldContext& WorldContext)
{
	FXRTrackingSystemBase::OnStartGameFrame(WorldContext);
	
	RefreshTrackingToWorldTransform(WorldContext);

	if (GameThreadFrame.IsValid())
	{
		if (GameThreadFrame->LightEstimate.bIsValid)
		{
			UARBasicLightEstimate* NewLightEstimate = NewObject<UARBasicLightEstimate>();
			NewLightEstimate->SetLightEstimate( GameThreadFrame->LightEstimate.AmbientIntensity,  GameThreadFrame->LightEstimate.AmbientColorTemperatureKelvin);
			LightEstimate = NewLightEstimate;
		}
		else
		{
			LightEstimate = nullptr;
		}
		
	}
	
	return true;
}

void* FAppleARKitSystem::GetARSessionRawPointer()
{
#if SUPPORTS_ARKIT_1_0
	return static_cast<void*>(Session);
#endif
	ensureAlwaysMsgf(false, TEXT("FAppleARKitSystem::GetARSessionRawPointer is unimplemented on current platform."));
	return nullptr;
}

void* FAppleARKitSystem::GetGameThreadARFrameRawPointer()
{
#if SUPPORTS_ARKIT_1_0
	if (GameThreadFrame.IsValid())
	{
		return GameThreadFrame->NativeFrame;
	}
	else
	{
		return nullptr;
	}
#endif
	ensureAlwaysMsgf(false, TEXT("FAppleARKitSystem::GetARGameThreadFrameRawPointer is unimplemented on current platform."));
	return nullptr;
}

//bool FAppleARKitSystem::ARLineTraceFromScreenPoint(const FVector2D ScreenPosition, TArray<FARTraceResult>& OutHitResults)
//{
//	const bool bSuccess = HitTestAtScreenPosition(ScreenPosition, EAppleARKitHitTestResultType::ExistingPlaneUsingExtent, OutHitResults);
//	return bSuccess;
//}

void FAppleARKitSystem::OnARSystemInitialized()
{
	// Register for device orientation changes
	FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.AddThreadSafeSP(this, &FAppleARKitSystem::OrientationChanged);
}

EARTrackingQuality FAppleARKitSystem::OnGetTrackingQuality() const
{
	return GameThreadFrame.IsValid()
		? GameThreadFrame->Camera.TrackingQuality
		: EARTrackingQuality::NotTracking;
}

EARTrackingQualityReason FAppleARKitSystem::OnGetTrackingQualityReason() const
{
	return GameThreadFrame.IsValid()
	? GameThreadFrame->Camera.TrackingQualityReason
	: EARTrackingQualityReason::None;
}

void FAppleARKitSystem::OnStartARSession(UARSessionConfig* SessionConfig)
{
	Run(SessionConfig);
}

void FAppleARKitSystem::OnPauseARSession()
{
	ensureAlwaysMsgf(false, TEXT("FAppleARKitSystem::OnPauseARSession() is unimplemented."));
}

void FAppleARKitSystem::OnStopARSession()
{
	Pause();
	ClearTrackedGeometries();
}

FARSessionStatus FAppleARKitSystem::OnGetARSessionStatus() const
{
	return IsRunning()
		? FARSessionStatus(EARSessionStatus::Running)
		: FARSessionStatus(EARSessionStatus::NotStarted);
}

void FAppleARKitSystem::OnSetAlignmentTransform(const FTransform& InAlignmentTransform)
{
	const FTransform& NewAlignmentTransform = InAlignmentTransform;
	
	// Update transform for all geometries
	for (auto GeoIt= TrackedGeometryGroups.CreateIterator(); GeoIt; ++GeoIt)
	{
		check(GeoIt.Value().TrackedGeometry);
		GeoIt.Value().TrackedGeometry->UpdateAlignmentTransform(NewAlignmentTransform);
	}
	
	// Update transform for all Pins
	for (UARPin* Pin : Pins)
	{
		Pin->UpdateAlignmentTransform(NewAlignmentTransform);
	}
}

static bool IsHitInRange( float UnrealHitDistance )
{
    // Skip results further than 5m or closer that 20cm from camera
	return 20.0f < UnrealHitDistance && UnrealHitDistance < 500.0f;
}

#if SUPPORTS_ARKIT_1_0

static UARTrackedGeometry* FindGeometryFromAnchor( ARAnchor* InAnchor, TMap<FGuid, FTrackedGeometryGroup>& Geometries )
{
	if (InAnchor != NULL)
	{
		const FGuid AnchorGUID = FAppleARKitConversion::ToFGuid( InAnchor.identifier );
		FTrackedGeometryGroup* Result = Geometries.Find(AnchorGUID);
		if (Result != nullptr)
		{
			return Result->TrackedGeometry;
		}
	}
	
	return nullptr;
}

#endif

TArray<FARTraceResult> FAppleARKitSystem::OnLineTraceTrackedObjects( const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels )
{
	const float WorldToMetersScale = GetWorldToMetersScale();
	TArray<FARTraceResult> Results;

	// Sanity check
	if (IsRunning())
	{
#if SUPPORTS_ARKIT_1_0
		
		TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> This = GetARCompositionComponent();
		
		@autoreleasepool
		{
			// Perform a hit test on the Session's last frame
			ARFrame* HitTestFrame = Session.currentFrame;
			if (HitTestFrame)
			{
				Results.Reserve(8);
				
				// Convert the screen position to normalised coordinates in the capture image space
				FVector2D NormalizedImagePosition = FAppleARKitCamera( HitTestFrame.camera ).GetImageCoordinateForScreenPosition( ScreenCoord, EAppleARKitBackgroundFitMode::Fill );
				switch (DeviceOrientation)
				{
					case EDeviceScreenOrientation::Portrait:
						NormalizedImagePosition = FVector2D( NormalizedImagePosition.Y, 1.0f - NormalizedImagePosition.X );
						break;
						
					case EDeviceScreenOrientation::PortraitUpsideDown:
						NormalizedImagePosition = FVector2D( 1.0f - NormalizedImagePosition.Y, NormalizedImagePosition.X );
						break;
						
					default:
					case EDeviceScreenOrientation::LandscapeRight:
						break;
						
					case EDeviceScreenOrientation::LandscapeLeft:
						NormalizedImagePosition = FVector2D(1.0f, 1.0f) - NormalizedImagePosition;
						break;
				};
				
				// GEngine->AddOnScreenDebugMessage(-1, 5.f, FColor::Blue, FString::Printf(TEXT("Hit Test At Screen Position: x: %f, y: %f"), NormalizedImagePosition.X, NormalizedImagePosition.Y));

				// First run hit test against existing planes with extents (converting & filtering results as we go)
				if (!!(TraceChannels & EARLineTraceChannels::PlaneUsingExtent) || !!(TraceChannels & EARLineTraceChannels::PlaneUsingBoundaryPolygon))
				{
					// First run hit test against existing planes with extents (converting & filtering results as we go)
					ARRaycastQuery* RaycastQuery = [HitTestFrame raycastQueryFromPoint: CGPointMake(NormalizedImagePosition.X, NormalizedImagePosition.Y) allowingTarget: ARRaycastTarget::ARRaycastTargetExistingPlaneGeometry alignment: ARRaycastTargetAlignment::ARRaycastTargetAlignmentAny];
					NSArray< ARRaycastResult* >* PlaneHitTestResults = [Session raycast: RaycastQuery];
					for (ARRaycastResult* HitTestResult in PlaneHitTestResults )
					{
						FVector CameraPos = FAppleARKitConversion::ToFTransform(HitTestFrame.camera.transform).GetTranslation();
						FVector IntersectionPos = FAppleARKitConversion::ToFTransform(HitTestResult.worldTransform).GetTranslation();
						const float UnrealHitDistance = FVector::Dist(CameraPos, IntersectionPos);
						if ( IsHitInRange( UnrealHitDistance ) )
						{
							// Hit result has passed and above filtering, add it to the list
							// Convert to Unreal's Hit Test result format
							Results.Add(FARTraceResult(This, UnrealHitDistance, EARLineTraceChannels::PlaneUsingExtent, FAppleARKitConversion::ToFTransform(HitTestResult.worldTransform), FindGeometryFromAnchor(HitTestResult.anchor, TrackedGeometryGroups)));
						}
					}
				}
				
				// If there were no valid results, fall back to hit testing against one shot plane
				if (!!(TraceChannels & EARLineTraceChannels::GroundPlane))
				{
					ARRaycastQuery* RaycastQuery = [HitTestFrame raycastQueryFromPoint : CGPointMake(NormalizedImagePosition.X, NormalizedImagePosition.Y) allowingTarget : ARRaycastTarget::ARRaycastTargetExistingPlaneInfinite alignment : ARRaycastTargetAlignment::ARRaycastTargetAlignmentHorizontal];
					NSArray< ARRaycastResult* >* PlaneHitTestResults = [Session raycast : RaycastQuery];
					for (ARRaycastResult* HitTestResult in PlaneHitTestResults )
					{
						FVector CameraPos = FAppleARKitConversion::ToFTransform(HitTestFrame.camera.transform).GetTranslation();
						FVector IntersectionPos = FAppleARKitConversion::ToFTransform(HitTestResult.worldTransform).GetTranslation();
						const float UnrealHitDistance = FVector::Dist(CameraPos, IntersectionPos);
						if ( IsHitInRange( UnrealHitDistance ) )
						{
							// Hit result has passed and above filtering, add it to the list
							// Convert to Unreal's Hit Test result format
							Results.Add(FARTraceResult(This, UnrealHitDistance, EARLineTraceChannels::GroundPlane, FAppleARKitConversion::ToFTransform(HitTestResult.worldTransform), FindGeometryFromAnchor(HitTestResult.anchor, TrackedGeometryGroups)));
						}
					}
				}
				
				// If there were no valid results, fall back further to hit testing against feature points
				if (!!(TraceChannels & EARLineTraceChannels::FeaturePoint))
				{
					// GEngine->AddOnScreenDebugMessage(-1, 5.f, FColor::Red, FString::Printf(TEXT("No results for plane hit test - reverting to feature points"), NormalizedImagePosition.X, NormalizedImagePosition.Y));
					
					ARRaycastQuery* RaycastQuery = [HitTestFrame raycastQueryFromPoint : CGPointMake(NormalizedImagePosition.X, NormalizedImagePosition.Y) allowingTarget : ARRaycastTarget::ARRaycastTargetEstimatedPlane alignment : ARRaycastTargetAlignment::ARRaycastTargetAlignmentAny];
					NSArray< ARRaycastResult* >*PlaneHitTestResults = [Session raycast : RaycastQuery];
					for (ARRaycastResult* HitTestResult in PlaneHitTestResults)
					{
						FVector CameraPos = FAppleARKitConversion::ToFTransform(HitTestFrame.camera.transform).GetTranslation();
						FVector IntersectionPos = FAppleARKitConversion::ToFTransform(HitTestResult.worldTransform).GetTranslation();
						const float UnrealHitDistance = FVector::Dist(CameraPos, IntersectionPos);
						if ( IsHitInRange( UnrealHitDistance ) )
						{
							// Hit result has passed and above filtering, add it to the list
							// Convert to Unreal's Hit Test result format
							Results.Add( FARTraceResult( This, UnrealHitDistance, EARLineTraceChannels::FeaturePoint, FAppleARKitConversion::ToFTransform( HitTestResult.worldTransform ), FindGeometryFromAnchor(HitTestResult.anchor, TrackedGeometryGroups) ) );
						}
					}
				}
			}
		}
#endif
	}
	
	if (Results.Num() > 1)
	{
		Results.Sort([](const FARTraceResult& A, const FARTraceResult& B)
		{
			return A.GetDistanceFromCamera() < B.GetDistanceFromCamera();
		});
	}
	
	return Results;
}

TArray<FARTraceResult> FAppleARKitSystem::OnLineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels)
{
	UE_LOGF(LogAppleARKit, Warning, "FAppleARKitSystem::OnLineTraceTrackedObjects(Start, End, TraceChannels) is currently unsupported.  No results will be returned.")
	TArray<FARTraceResult> EmptyResults;
	return EmptyResults;
}

TArray<UARTrackedGeometry*> FAppleARKitSystem::OnGetAllTrackedGeometries() const
{
	TArray<UARTrackedGeometry*> Geometries;

	//TrackedGeometries.GenerateValueArray(Geometries);
	for (auto GeoIt = TrackedGeometryGroups.CreateConstIterator(); GeoIt; ++GeoIt)
	{
		Geometries.Add(GeoIt.Value().TrackedGeometry);
	}

	return Geometries;
}

TArray<UARPin*> FAppleARKitSystem::OnGetAllPins() const
{
	return ObjectPtrDecay(Pins);
}

UARTexture* FAppleARKitSystem::OnGetARTexture(EARTextureType TextureType) const
{
	auto Camera = static_cast<FAppleARKitXRCamera*>(XRCamera.Get());
	
	switch (TextureType)
	{
		case EARTextureType::CameraImage:
			return CameraImage;
			
		case EARTextureType::CameraDepth:
			return CameraDepth;
			
		case EARTextureType::PersonSegmentationImage:
			return Camera ? Camera->GetOverlay().GetOcclusionMatteTexture() : nullptr;
			
		case EARTextureType::PersonSegmentationDepth:
			return Camera ? Camera->GetOverlay().GetOcclusionDepthTexture() : nullptr;
		
		case EARTextureType::SceneDepthMap:
			return SceneDepthMap;
			
		case EARTextureType::SceneDepthConfidenceMap:
			return SceneDepthConfidenceMap;
	}
	
	return nullptr;
}

UARLightEstimate* FAppleARKitSystem::OnGetCurrentLightEstimate() const
{
	return LightEstimate;
}

UARPin* FAppleARKitSystem::FindARPinByComponent(const USceneComponent* Component) const
{
	return ARKitUtil::PinFromComponent(Component, ObjectPtrDecay(Pins));
}

UARPin* FAppleARKitSystem::OnPinComponent( USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry, const FName DebugName )
{
	if ( ensureMsgf(ComponentToPin != nullptr, TEXT("Cannot pin component.")) )
	{
		if (UARPin* FindResult = ARKitUtil::PinFromComponent(ComponentToPin, ObjectPtrDecay(Pins)))
		{
			UE_LOGF(LogAppleARKit, Warning, "Component %ls is already pinned. Unpin it first.", *ComponentToPin->GetReadableName());
			OnRemovePin(FindResult);
		}

		// PinToWorld * AlignedTrackingToWorld(-1) * TrackingToAlignedTracking(-1) = PinToWorld * WorldToAlignedTracking * AlignedTrackingToTracking
		// The Worlds and AlignedTracking cancel out, and we get PinToTracking
		// But we must translate this logic into Unreal's transform API
		const FTransform& TrackingToAlignedTracking = GetARCompositionComponent()->GetAlignmentTransform();
		const FTransform PinToTrackingTransform = PinToWorldTransform.GetRelativeTransform(GetTrackingToWorldTransform()).GetRelativeTransform(TrackingToAlignedTracking);

		// If the user did not provide a TrackedGeometry, create the simplest TrackedGeometry for this pin.
		UARTrackedGeometry* GeometryToPinTo = TrackedGeometry;
        void* NativeResource = nullptr;
		if (GeometryToPinTo == nullptr)
		{
			double UpdateTimestamp = FPlatformTime::Seconds();
			
			GeometryToPinTo = NewObject<UARTrackedPoint>();
			GeometryToPinTo->UpdateTrackedGeometry(GetARCompositionComponent().ToSharedRef(), 0, FPlatformTime::Seconds(), PinToTrackingTransform, GetARCompositionComponent()->GetAlignmentTransform());
			
#if SUPPORTS_ARKIT_1_0
			if (FAppleARKitAvailability::SupportsARKit10())
			{
				const auto TrackingToWorldTransform = GetTrackingToWorldTransform();
				const auto AnchorTransform = PinToWorldTransform * TrackingToWorldTransform.Inverse() * TrackingToAlignedTracking.Inverse();
				auto Anchor = [[ARAnchor alloc] initWithTransform: FAppleARKitConversion::ToARKitMatrix(AnchorTransform)];
				auto AnchorGUID = FAppleARKitConversion::ToFGuid(Anchor.identifier);
				
				FTrackedGeometryGroup TrackedGeometryGroup(GeometryToPinTo);
				TrackedGeometryGroups.Add(AnchorGUID, TrackedGeometryGroup);
				
				{
					FScopeLock ScopeLock(&AnchorsLock);
					AllAnchors.Add(AnchorGUID, Anchor);
				}
				
				NativeResource = static_cast<void*>(Anchor);
			}
#endif
		}
		
		UARPin* NewPin = NewObject<UARPin>();
		NewPin->InitARPin(GetARCompositionComponent().ToSharedRef(), ComponentToPin, PinToTrackingTransform, GeometryToPinTo, DebugName);
		
		if (NativeResource != nullptr)
		{
			NewPin->SetNativeResource(NativeResource);
		}
		
		Pins.Add(NewPin);
		
		return NewPin;
	}
	else
	{
		return nullptr;
	}
}

void FAppleARKitSystem::OnRemovePin(UARPin* PinToRemove)
{
#if SUPPORTS_ARKIT_1_0
	// Release the underlying ARKit anchor for this pin.
	{
		ARAnchor* Anchor = static_cast<ARAnchor*>(PinToRemove->GetNativeResource());
		if (Anchor)
		{
			FScopeLock ScopeLock(&AnchorsLock);

			const auto Guid = FAppleARKitConversion::ToFGuid(Anchor.identifier);
			if (auto Record = AllAnchors.Find(Guid))
			{
				ARAnchor* SavedAnchor = *Record;
				check(SavedAnchor == Anchor); // SavedAnchor should be the same as Anchor.
				[SavedAnchor release] ;
				AllAnchors.Remove(Guid);
			}
		}
	}
#endif

	Pins.RemoveSingleSwap(PinToRemove);
}

bool FAppleARKitSystem::OnIsTrackingTypeSupported(EARSessionType SessionType) const
{
#if SUPPORTS_ARKIT_1_0
	switch (SessionType)
	{
		case EARSessionType::Orientation:
		{
			return AROrientationTrackingConfiguration.isSupported == TRUE;
		}
		case EARSessionType::World:
		{
			return ARWorldTrackingConfiguration.isSupported == TRUE;
		}
		case EARSessionType::Face:
		{
			if (auto Implementation = GetModularInterface<IAppleARKitFaceSupport>())
			{
				return Implementation->DoesSupportFaceAR();
			}
			return false;
		}

#if SUPPORTS_ARKIT_2_0
		case EARSessionType::Image:
		{
			return ARImageTrackingConfiguration.isSupported == TRUE;
		}
		case EARSessionType::ObjectScanning:
		{
			return ARObjectScanningConfiguration.isSupported == TRUE;
		}
#endif

#if SUPPORTS_ARKIT_3_0
		case EARSessionType::PoseTracking:
		{
			return ARBodyTrackingConfiguration.isSupported == TRUE;
		}
#endif

#if SUPPORTS_ARKIT_4_0
		case EARSessionType::GeoTracking:
		{
			return ARGeoTrackingConfiguration.isSupported == TRUE;
		}
#endif

	}
#endif
	return false;
}

bool FAppleARKitSystem::OnAddManualEnvironmentCaptureProbe(FVector Location, FVector Extent)
{
#if SUPPORTS_ARKIT_2_0
	if (Session != nullptr)
	{
		if (FAppleARKitAvailability::SupportsARKit20())
		{
//@joeg -- Todo need to fix this transform as it needs to use the alignment transform too
			// Build and add the anchor
			simd_float4x4 AnchorMatrix = FAppleARKitConversion::ToARKitMatrix(FTransform(Location));
			simd_float3 AnchorExtent = FAppleARKitConversion::ToARKitVector(Extent * 2.f);
			AREnvironmentProbeAnchor* ARProbe = [[AREnvironmentProbeAnchor alloc] initWithTransform: AnchorMatrix extent: AnchorExtent];
			[Session addAnchor: ARProbe];
			[ARProbe release];
		}
		return true;
	}
#endif
	return false;
}

TArray<FARVideoFormat> FAppleARKitSystem::OnGetSupportedVideoFormats(EARSessionType SessionType) const
{
#if SUPPORTS_ARKIT_1_5
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		return FAppleARKitConversion::FromARVideoFormatArray(FAppleARKitConversion::GetSupportedVideoFormats(SessionType));
	}
#endif
	return TArray<FARVideoFormat>();
}

TArray<FVector> FAppleARKitSystem::OnGetPointCloud() const
{
	TArray<FVector> PointCloud;
	
#if SUPPORTS_ARKIT_1_0
	if (GameThreadFrame.IsValid())
	{
		ARFrame* InARFrame = (ARFrame*)GameThreadFrame->NativeFrame;
		ARPointCloud* InARPointCloud = InARFrame.rawFeaturePoints;
		if (InARPointCloud != nullptr)
		{
			const int32 Count = InARPointCloud.count;
			PointCloud.Empty(Count);
			PointCloud.AddUninitialized(Count);
			for (int32 Index = 0; Index < Count; Index++)
			{
				PointCloud[Index] = FAppleARKitConversion::ToFVector(InARPointCloud.points[Index]);
			}
		}
	}
#endif
	return PointCloud;
}

#if SUPPORTS_ARKIT_2_0
/** Since both the object extraction and world saving need to get the world map async, use a common chunk of code for this */
class FAppleARKitGetWorldMapObjectAsyncTask
{
public:
	/** Performs the call to get the world map and triggers the OnWorldMapAcquired() the completion handler */
	void Run()
	{
		[Session getCurrentWorldMapWithCompletionHandler: ^(ARWorldMap* worldMap, NSError* error)
		 {
			 WorldMap = worldMap;
			 [WorldMap retain];
			 bool bWasSuccessful = error == nullptr;
			 FString ErrorString;
			 if (error != nullptr)
			 {
				 ErrorString = [error localizedDescription];
			 }
			 OnWorldMapAcquired(bWasSuccessful, ErrorString);
		 }];
	}
	
protected:
	FAppleARKitGetWorldMapObjectAsyncTask(ARSession* InSession) :
		Session(InSession)
	{
		CFRetain(Session);
	}
	
	void Release()
	{
		if (Session != nullptr)
		{
			[Session release];
			Session = nullptr;
		}
		if (WorldMap != nullptr)
		{
			[WorldMap release];
			WorldMap = nullptr;
		}
	}

	/** Called once the world map completion handler is called */
	virtual void OnWorldMapAcquired(bool bWasSuccessful, FString ErrorString) = 0;

	/** The session object that we'll grab the world from */
	ARSession* Session;
	/** The world map object once the call has completed */
	ARWorldMap* WorldMap;
};

//@joeg -- The API changed last minute so you don't need to resolve the world to get an object anymore
// This needs to be cleaned up
class FAppleARKitGetCandidateObjectAsyncTask :
	public FARGetCandidateObjectAsyncTask
{
public:
	FAppleARKitGetCandidateObjectAsyncTask(ARSession* InSession, FVector InLocation, FVector InExtent) :
		Location(InLocation)
		, Extent(InExtent)
		, ReferenceObject(nullptr)
		, Session(InSession)
	{
		[Session retain];
	}
	
	/** @return the candidate object that you can use for detection later */
	virtual UARCandidateObject* GetCandidateObject() override
	{
		if (ReferenceObject != nullptr)
		{
			UARCandidateObject* CandidateObject = NewObject<UARCandidateObject>();
			
			FVector RefObjCenter = FAppleARKitConversion::ToFVector(ReferenceObject.center);
			FVector RefObjExtent = 0.5f * FAppleARKitConversion::ToFVector(ReferenceObject.extent);
			FBox BoundingBox(RefObjCenter, RefObjExtent);
			CandidateObject->SetBoundingBox(BoundingBox);
			
			// Serialize the object into a byte array and stick that on the candidate object
			NSError* ErrorObj = nullptr;
			NSData* RefObjData = [NSKeyedArchiver archivedDataWithRootObject: ReferenceObject requiringSecureCoding: YES error: &ErrorObj];
			uint32 SavedSize = RefObjData.length;
			TArray<uint8> RawBytes;
			RawBytes.AddUninitialized(SavedSize);
			FPlatformMemory::Memcpy(RawBytes.GetData(), [RefObjData bytes], SavedSize);
			CandidateObject->SetCandidateObjectData(RawBytes);

			return CandidateObject;
		}
		return nullptr;
	}
	
	virtual ~FAppleARKitGetCandidateObjectAsyncTask()
	{
		[Session release];
		if (ReferenceObject != nullptr)
		{
			CFRelease(ReferenceObject);
		}
	}

	void Run()
	{
		simd_float4x4 ARMatrix = FAppleARKitConversion::ToARKitMatrix(FTransform(Location));
		simd_float3 Center = 0.f;
		simd_float3 ARExtent = FAppleARKitConversion::ToARKitVector(Extent * 2.f);

		[Session createReferenceObjectWithTransform: ARMatrix center: Center extent: ARExtent
		 completionHandler: ^(ARReferenceObject* refObject, NSError* error)
		{
			ReferenceObject = refObject;
			CFRetain(ReferenceObject);
			bool bWasSuccessful = error == nullptr;
			bHadError = error != nullptr;
			FString ErrorString;
			if (error != nullptr)
			{
				ErrorString = [error localizedDescription];
			}
			bIsDone = true;
		}];
	}
	
private:
	FVector Location;
	FVector Extent;
	ARReferenceObject* ReferenceObject;

	/** The session object that we'll grab the object from */
	ARSession* Session;
};

class FAppleARKitSaveWorldAsyncTask :
	public FARSaveWorldAsyncTask,
	public FAppleARKitGetWorldMapObjectAsyncTask
{
public:
	FAppleARKitSaveWorldAsyncTask(ARSession* InSession, const FTransform& InAlignmentTransform) :
		FAppleARKitGetWorldMapObjectAsyncTask(InSession),
		AlignmentTransform(InAlignmentTransform)
	{
	}

	virtual ~FAppleARKitSaveWorldAsyncTask()
	{
		Release();
	}

	virtual void OnWorldMapAcquired(bool bWasSuccessful, FString ErrorString) override
	{
		if (bWasSuccessful)
		{
			NSError* ErrorObj = nullptr;
			NSData* WorldNSData = [NSKeyedArchiver archivedDataWithRootObject: WorldMap requiringSecureCoding: YES error: &ErrorObj];
			if (ErrorObj == nullptr)
			{
				/**
				 World map buffer layout:
				 FARWorldSaveHeader: uncompressed
				 FTransform:         uncompressed
				 world map blob:     compressed
				 */
				const auto UncompressedSize = WorldNSData.length;
				const auto HeaderAndTransformSize = AR_SAVE_WORLD_HEADER_SIZE + sizeof(FTransform);
				TArray<uint8> CompressedData;
				CompressedData.AddUninitialized(HeaderAndTransformSize + UncompressedSize);
				uint8* Buffer = (uint8*)CompressedData.GetData();
				
				// Write our magic header into our buffer
				FARWorldSaveHeader& Header = *(FARWorldSaveHeader*)Buffer;
				Header = FARWorldSaveHeader();
				Header.UncompressedSize = UncompressedSize;
				
				// Wirte the alignment transform
				*(FTransform*)(Buffer + AR_SAVE_WORLD_HEADER_SIZE) = AlignmentTransform;
				
				// Compress the data
				uint8* CompressInto = Buffer + HeaderAndTransformSize;
				int32 CompressedSize = UncompressedSize;
				uint8* UncompressedData = (uint8*)[WorldNSData bytes];
				verify(FCompression::CompressMemory(NAME_Zlib, CompressInto, CompressedSize, UncompressedData, UncompressedSize));
				
				// Only copy out the amount of compressed data, the header and the alignment transform
				int32 CompressedSizePlusHeader = HeaderAndTransformSize + CompressedSize;
				WorldData.AddUninitialized(CompressedSizePlusHeader);
				FPlatformMemory::Memcpy(WorldData.GetData(), CompressedData.GetData(), CompressedSizePlusHeader);
			}
			else
			{
				Error = [ErrorObj localizedDescription];
				bHadError = true;
			}
		}
		else
		{
			Error = ErrorString;
			bHadError = true;
		}
		// Trigger that we're done
		bIsDone = true;
	}
	
private:
	FTransform AlignmentTransform = FTransform::Identity;
};
#endif

TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> FAppleARKitSystem::OnGetCandidateObject(FVector Location, FVector Extent) const
{
#if SUPPORTS_ARKIT_2_0
	if (Session != nullptr)
	{
		if (FAppleARKitAvailability::SupportsARKit20())
		{
			TSharedPtr<FAppleARKitGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> Task = MakeShared<FAppleARKitGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>(Session, Location, Extent);
			Task->Run();
			return Task;
		}
	}
#endif
	return  MakeShared<FARErrorGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>(TEXT("GetCandidateObject - requires a valid, running ARKit 2.0 session"));
}

TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> FAppleARKitSystem::OnSaveWorld() const
{
#if SUPPORTS_ARKIT_2_0
	if (Session != nullptr)
	{
		if (FAppleARKitAvailability::SupportsARKit20())
		{
			const auto AlignmentTransform = GetARCompositionComponent()->GetAlignmentTransform();
			TSharedPtr<FAppleARKitSaveWorldAsyncTask, ESPMode::ThreadSafe> Task = MakeShared<FAppleARKitSaveWorldAsyncTask, ESPMode::ThreadSafe>(Session, AlignmentTransform);
			Task->Run();
			return Task;
		}
	}
#endif
	return  MakeShared<FARErrorSaveWorldAsyncTask, ESPMode::ThreadSafe>(TEXT("SaveWorld - requires a valid, running ARKit 2.0 session"));
}

EARWorldMappingState FAppleARKitSystem::OnGetWorldMappingStatus() const
{
	if (GameThreadFrame.IsValid())
	{
		return GameThreadFrame->WorldMappingState;
	}
	return EARWorldMappingState::NotAvailable;
}


void FAppleARKitSystem::AddReferencedObjects( FReferenceCollector& Collector )
{
	for (auto GeoIt = TrackedGeometryGroups.CreateIterator(); GeoIt; ++GeoIt)
	{
		FTrackedGeometryGroup& TrackedGeometryGroup = GeoIt.Value();

		Collector.AddReferencedObject(TrackedGeometryGroup.TrackedGeometry);
		Collector.AddReferencedObject(TrackedGeometryGroup.ARActor);
		Collector.AddReferencedObject(TrackedGeometryGroup.ARComponent);
	}

	if (Pins.Num())
	{
		Collector.AddReferencedObjects(Pins);
	}
	if (CameraImage)
	{
		Collector.AddReferencedObject(CameraImage);
	}
	if (CameraDepth)
	{
		Collector.AddReferencedObject(CameraDepth);
	}
	if (CandidateImages.Num())
	{
		Collector.AddReferencedObjects(CandidateImages);
	}
	if (CandidateObjects.Num())
	{
		Collector.AddReferencedObjects(CandidateObjects);
	}
	if (TimecodeProvider)
	{
		Collector.AddReferencedObject(TimecodeProvider);
	}
	if (SceneDepthMap)
	{
		Collector.AddReferencedObject(SceneDepthMap);
	}
	if (SceneDepthConfidenceMap)
	{
		Collector.AddReferencedObject(SceneDepthConfidenceMap);
	}
	if (LightEstimate)
	{
		Collector.AddReferencedObject(LightEstimate);
	}
}

void FAppleARKitSystem::SetDeviceOrientationAndDerivedTracking(EDeviceScreenOrientation InOrientation)
{
	ensureAlwaysMsgf(InOrientation != EDeviceScreenOrientation::Unknown, TEXT("statusBarOrientation should only ever return valid orientations"));
	if (InOrientation == EDeviceScreenOrientation::Unknown)
	{
		// This is the default for AR apps
		InOrientation = EDeviceScreenOrientation::LandscapeLeft;
	}

	// even if this didn't change, we need to call CalcTrackingToWorldRotation because the camera mode may have changed (from Gravity to non-Gravity, etc)

	DeviceOrientation = InOrientation;
	CalcTrackingToWorldRotation();
	
}

void FAppleARKitSystem::ClearTrackedGeometries()
{
#if SUPPORTS_ARKIT_1_0
	TArray<FGuid> Keys;
	for (auto GeoIt = TrackedGeometryGroups.CreateIterator(); GeoIt; ++GeoIt)
	{
		Keys.Add(GeoIt.Key());
	}
	//TrackedGeometries.GetKeys(Keys);
	for (const FGuid& Key : Keys)
	{
		SessionDidRemoveAnchors_Internal(Key);
	}
	
	// Clear all the saved anchors
	{
		FScopeLock ScopeLock(&AnchorsLock);
		for (auto Itr : AllAnchors)
		{
			[Itr.Value release];
		}
		AllAnchors = {};
	}
#endif
	
	FARKitMeshData::ClearAllMeshData();
}

void FAppleARKitSystem::SetupCameraTextures()
{
#if SUPPORTS_ARKIT_1_0
	if (CameraImage == nullptr)
	{
		CameraImage = NewObject<UAppleARKitCameraVideoTexture>();
		CameraImage->Init();
		FAppleARKitXRCamera* Camera = GetARKitXRCamera();
		check(Camera);
		Camera->GetOverlay().SetOverlayTexture(CameraImage);
	}
	
	if (CameraDepth == nullptr)
	{
		CameraDepth = NewObject<UAppleARKitTextureCameraDepth>();
	}
#endif
}

UARTrackedGeometry* FAppleARKitSystem::TryCreateTrackedGeometry(TSharedRef<FAppleARKitAnchorData> AnchorData)
{
#if SUPPORTS_ARKIT_1_0
    double UpdateTimestamp = FPlatformTime::Seconds();
    
    const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& ARComponent = GetARCompositionComponent();
    const UARSessionConfig& SessionConfig = ARComponent->GetSessionConfig();
    const FTransform& AlignmentTransform = ARComponent->GetAlignmentTransform();
    const auto AnchorWorldTransform = AnchorData->Transform * AlignmentTransform * GetTrackingToWorldTransform();
    
    FString NewAnchorDebugName;
    UARTrackedGeometry* NewGeometry = nullptr;
    UClass* ARComponentClass = nullptr;
    
    switch (AnchorData->AnchorType)
    {
        case EAppleAnchorType::Anchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("ANCHOR-%02d"), LastTrackedGeometry_DebugId++);
            NewGeometry = NewObject<UARTrackedPoint>();
            NewGeometry->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform);
            ARComponentClass = SessionConfig.GetPointComponentClass();
            break;
        }
        case EAppleAnchorType::PlaneAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("PLN-%02d"), LastTrackedGeometry_DebugId++);
            UARPlaneGeometry* NewGeo = NewObject<UARPlaneGeometry>();
            NewGeo->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->Center, AnchorData->Extent, AnchorData->BoundaryVerts, nullptr);
            NewGeo->SetOrientation(AnchorData->Orientation);
            NewGeo->SetObjectClassification(AnchorData->ObjectClassification);
            NewGeometry = NewGeo;
            ARComponentClass = SessionConfig.GetPlaneComponentClass();
            break;
        }
        case EAppleAnchorType::FaceAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("FACE-%02d"), LastTrackedGeometry_DebugId++);
            UARFaceGeometry* NewGeo = NewObject<UARFaceGeometry>();
            NewGeo->UpdateFaceGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->BlendShapes, AnchorData->FaceVerts, AnchorData->FaceIndices, AnchorData->FaceUVData, AnchorData->LeftEyeTransform, AnchorData->RightEyeTransform, AnchorData->LookAtTarget);
            NewGeometry = NewGeo;
            ARComponentClass = SessionConfig.GetFaceComponentClass();
            break;
        }
        case EAppleAnchorType::ImageAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("IMG-%02d"), LastTrackedGeometry_DebugId++);
            UARTrackedImage* NewImage = NewObject<UARTrackedImage>();
            auto* CandidateImage = CandidateImages.Find(AnchorData->DetectedAnchorName);
            ensure(CandidateImage != nullptr);
            FVector2D PhysicalSize((*CandidateImage)->GetPhysicalWidth(), (*CandidateImage)->GetPhysicalHeight());
            NewImage->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, PhysicalSize, *CandidateImage);
            NewGeometry = NewImage;
            ARComponentClass = SessionConfig.GetImageComponentClass();
            break;
        }
        case EAppleAnchorType::EnvironmentProbeAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("ENV-%02d"), LastTrackedGeometry_DebugId++);
            UAppleARKitEnvironmentCaptureProbe* NewProbe = NewObject<UAppleARKitEnvironmentCaptureProbe>();
            NewProbe->UpdateEnvironmentCapture(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->Extent, AnchorData->ProbeTexture);
            NewGeometry = NewProbe;
            ARComponentClass = SessionConfig.GetEnvironmentProbeComponentClass();
            break;
        }
        case EAppleAnchorType::ObjectAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("OBJ-%02d"), LastTrackedGeometry_DebugId++);
            UARTrackedObject* NewTrackedObject = NewObject<UARTrackedObject>();
            auto* CandidateObject = CandidateObjects.Find(AnchorData->DetectedAnchorName);
            ensure(CandidateObject != nullptr);
            NewTrackedObject->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, GetARCompositionComponent()->GetAlignmentTransform(), *CandidateObject);
            NewGeometry = NewTrackedObject;
            ARComponentClass = SessionConfig.GetObjectComponentClass();
            break;
        }
        case EAppleAnchorType::PoseAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("POSE-%02d"), LastTrackedGeometry_DebugId++);
            UARTrackedPose* NewTrackedPose = NewObject<UARTrackedPose>();
            NewTrackedPose->UpdateTrackedPose(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->TrackedPose);
            NewGeometry = NewTrackedPose;
            ARComponentClass = SessionConfig.GetPoseComponentClass();
            break;
        }
        case EAppleAnchorType::MeshAnchor:
        {
            NewAnchorDebugName = FString::Printf(TEXT("MESH-%02d"), LastTrackedGeometry_DebugId++);
            UARMeshGeometry* NewGeo = NewObject<UARKitMeshGeometry>();
            NewGeo->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform);
            
            NewGeometry = NewGeo;
            ARComponentClass = SessionConfig.GetMeshComponentClass();
            break;
        }
		case EAppleAnchorType::GeoAnchor:
		{
			NewAnchorDebugName = FString::Printf(TEXT("GEO-%02d"), LastTrackedGeometry_DebugId++);
			UARGeoAnchor* NewGeo = NewObject<UARGeoAnchor>();
			NewGeo->UpdateGeoAnchor(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform,
									AnchorData->Longitude, AnchorData->Latitude, AnchorData->AltitudeMeters, AnchorData->AltitudeSource);
			
			NewGeometry = NewGeo;
			ARComponentClass = SessionConfig.GetGeoAnchorComponentClass();
			break;
		}
    }
    check(NewGeometry != nullptr);

	NewGeometry->SetTrackingState(AnchorData->bIsTracked ? EARTrackingState::Tracking : EARTrackingState::NotTracking);
    NewGeometry->UniqueId = AnchorData->AnchorGUID;
    NewGeometry->SetDebugName( FName(*NewAnchorDebugName) );
    NewGeometry->SetName(AnchorData->AnchorName);
    
    FTrackedGeometryGroup TrackedGeometryGroup(NewGeometry);
    TrackedGeometryGroups.Add(AnchorData->AnchorGUID, TrackedGeometryGroup);
    
    AARActor::RequestSpawnARActor(AnchorData->AnchorGUID, ARComponentClass);
    return NewGeometry;
#else
    return nullptr;
#endif
}

void FAppleARKitSystem::OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID)
{
	FTrackedGeometryGroup* TrackedGeometryGroup = TrackedGeometryGroups.Find(NativeID);
	if (TrackedGeometryGroup != nullptr)
	{
		//this should still be null
		check(TrackedGeometryGroup->ARActor == nullptr);
		check(TrackedGeometryGroup->ARComponent == nullptr);

		check(NewARActor);
		check(NewARComponent);

		TrackedGeometryGroup->ARActor = NewARActor;
		TrackedGeometryGroup->ARComponent = NewARComponent;

		//NOW, we can make the callbacks
		TrackedGeometryGroup->ARComponent->Update(TrackedGeometryGroup->TrackedGeometry);
		TriggerOnTrackableAddedDelegates(TrackedGeometryGroup->TrackedGeometry);
	}
	else
	{
		UE_LOGF(LogAppleARKit, Warning, "AR NativeID not found.  Make sure to set this on the ARComponent!");
	}
}

bool FAppleARKitSystem::Run(UARSessionConfig* SessionConfig)
{
	TimecodeProvider = UAppleARKitSettings::GetTimecodeProvider();

	SetupCameraTextures();

	{
		// Clear out any existing frames since they aren't valid anymore
		FScopeLock ScopeLock(&FrameLock);
		GameThreadFrame = TSharedPtr<FAppleARKitFrame, ESPMode::ThreadSafe>();
		LastReceivedFrame = TSharedPtr<FAppleARKitFrame, ESPMode::ThreadSafe>();
	}
	
	bool bPersonOcclusionEnabled = false;
	bool bSceneDepthEnabled = false;
	
	// The initial alignment transform that's saved along with the world map data
	TOptional<FTransform> InitialAlignmentTransform;
	
#if SUPPORTS_ARKIT_1_0
	// Don't do the conversion work if they don't want this
	FAppleARKitAnchorData::bGenerateGeometry = SessionConfig->bGenerateMeshDataFromTrackedGeometry;

	if (FAppleARKitAvailability::SupportsARKit10())
	{
		ARSessionRunOptions options = 0;

		ARConfiguration* Configuration = nullptr;
		CheckForFaceARSupport(SessionConfig);
		CheckForPoseTrackingARLiveLink(SessionConfig);
		if (FaceARSupport != nullptr && SessionConfig->GetSessionType() == EARSessionType::Face)
		{
			Configuration = FaceARSupport->ToARConfiguration(SessionConfig, TimecodeProvider);
		}
		else
		{
			Configuration = FAppleARKitConversion::ToARConfiguration(SessionConfig, MutableView(CandidateImages), ConvertedCandidateImages, MutableView(CandidateObjects), InitialAlignmentTransform);
		}

		// Not all session types are supported by all devices
		if (Configuration == nullptr)
		{
			UE_LOGF(LogAppleARKit, Error, "The requested session type is not supported by this device");
			return false;
		}
		
		// Configure additional tracking features
		FAppleARKitConversion::ConfigureSessionTrackingFeatures(SessionConfig, Configuration);

		// Create our ARSessionDelegate
		if (Delegate == nullptr)
		{
			Delegate = [[FAppleARKitSessionDelegate alloc] initWithAppleARKitSystem:this];
		}
		
		// Create MetalTextureCache
		if (IsMetalPlatform(GMaxRHIShaderPlatform))
		{
            id<MTLDevice> Device = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
			check(Device);

			CVReturn Return = CVMetalTextureCacheCreate(nullptr, nullptr, (__bridge id<MTLDevice>)Device, nullptr, &MetalTextureCache);
			check(Return == kCVReturnSuccess);
			check(MetalTextureCache);

			// Pass to session delegate to use for Metal texture creation
			[Delegate setMetalTextureCache : MetalTextureCache];
		}
		
		if (Session == nullptr)
		{
			// Start a new ARSession
			Session = [ARSession new];
			Session.delegate = Delegate;
			Session.delegateQueue = dispatch_get_global_queue(QOS_CLASS_USER_INTERACTIVE, 0);
		}
		else
		{
			// Check what the user has set for reseting options
			if (SessionConfig->ShouldResetCameraTracking())
			{
				options |= ARSessionRunOptionResetTracking;
			}
			if (SessionConfig->ShouldResetTrackedObjects())
			{
				options |= ARSessionRunOptionRemoveExistingAnchors;
				// The user requested us to remove existing anchors so remove ours now
				ClearTrackedGeometries();
			}
		}
		
#if PLATFORM_IOS && !PLATFORM_TVOS
		// Check if we need to adjust the priorities to allow ARKit to have more CPU time
		if (GetMutableDefault<UAppleARKitSettings>()->ShouldAdjustThreadPriorities())
		{
			int32 GameOverride = GetMutableDefault<UAppleARKitSettings>()->GetGameThreadPriorityOverride();
			int32 RenderOverride = GetMutableDefault<UAppleARKitSettings>()->GetRenderThreadPriorityOverride();
			SetThreadPriority(GameOverride);
			if (XRCamera.IsValid())
			{
				FAppleARKitXRCamera* Camera = (FAppleARKitXRCamera*)XRCamera.Get();
				Camera->AdjustThreadPriority(RenderOverride);
			}
			
			UE_LOGF(LogAppleARKit, Log, "Overriding thread priorities: Game Thread (%d), Render Thread (%d)", GameOverride, RenderOverride);
		}
#endif
		
#if SUPPORTS_ARKIT_3_0
		if (FAppleARKitAvailability::SupportsARKit30())
		{
			if (Configuration.frameSemantics == ARFrameSemanticPersonSegmentation || Configuration.frameSemantics == ARFrameSemanticPersonSegmentationWithDepth)
			{
				bPersonOcclusionEnabled = true;
			}
		}
		
		if (Session && SessionConfig->bUseStandardOnboardingUX)
		{
			if (!CoachingOverlay)
			{
				CoachingOverlay = [[FARKitCoachingOverlay alloc] initWithAppleARKitSystem: this];
			}
			
			[CoachingOverlay setARSession: Session];
			[CoachingOverlay addToRootView];
		}
#endif
		
#if SUPPORTS_ARKIT_4_0
		if (FAppleARKitAvailability::SupportsARKit40())
		{
			if (Configuration.frameSemantics == ARFrameSemanticSceneDepth || Configuration.frameSemantics == ARFrameSemanticSmoothedSceneDepth)
			{
				bSceneDepthEnabled = true;
			}
		}
#endif

		UE_LOGF(LogAppleARKit, Log, "Starting session: %p with options %lld", this, options);

		// Start the session with the configuration
		[Session runWithConfiguration : Configuration options : options];
	}
	
#endif // SUPPORTS_ARKIT_1_0
	
	if (FAppleARKitXRCamera* Camera = GetARKitXRCamera())
    {
		if (SessionConfig->bUsePersonSegmentationForOcclusion && bPersonOcclusionEnabled)
		{
			Camera->GetOverlay().SetOcclusionType(EARKitOcclusionType::PersonSegmentation);
		}
		else if (SessionConfig->bUseSceneDepthForOcclusion && bSceneDepthEnabled)
		{
			Camera->GetOverlay().SetOcclusionType(EARKitOcclusionType::SceneDepth);
		}
    }

	// Make sure this is set at session start, because there are timing issues with using only the delegate approach
	// Also this needs to be set each time a new session is started in case we switch tracking modes (gravity vs face)
	EDeviceScreenOrientation ScreenOrientation = FPlatformMisc::GetDeviceOrientation();
	SetDeviceOrientationAndDerivedTracking(ScreenOrientation);

	// @todo arkit Add support for relocating ARKit space to Unreal World Origin? BaseTransform = FTransform::Identity;
	
	// Set running state
	bIsRunning = true;
	
	GetARCompositionComponent()->OnARSessionStarted.Broadcast();
	
	// Apply the initial alignment transform if it's available
	if (InitialAlignmentTransform)
	{
		UARBlueprintLibrary::SetAlignmentTransform(InitialAlignmentTransform.GetValue());
	}
	return true;
}

bool FAppleARKitSystem::IsRunning() const
{
	return bIsRunning;
}

bool FAppleARKitSystem::Pause()
{
	// Already stopped?
	if (!IsRunning())
	{
		return true;
	}
	
	UE_LOGF(LogAppleARKit, Log, "Stopping session: %p", this);

#if SUPPORTS_ARKIT_1_0
	if (FAppleARKitAvailability::SupportsARKit10())
	{
		// Suspend the session
		[Session pause];
		
		// Release MetalTextureCache created in Start
		if (MetalTextureCache)
		{
			// Tell delegate to release it
			[Delegate setMetalTextureCache:nullptr];
		
			CFRelease(MetalTextureCache);
			MetalTextureCache = nullptr;
		}
		
		if (CVarReleaseSessionWhenStopped.GetValueOnAnyThread())
		{
			[Session release];
			Session = nullptr;
		}
	}
	
#if PLATFORM_IOS && !PLATFORM_TVOS
	// Check if we need to adjust the priorities to allow ARKit to have more CPU time
	if (GetMutableDefault<UAppleARKitSettings>()->ShouldAdjustThreadPriorities())
	{
		SetThreadPriority(GAME_THREAD_PRIORITY);
		if (XRCamera.IsValid())
		{
			FAppleARKitXRCamera* Camera = (FAppleARKitXRCamera*)XRCamera.Get();
			Camera->AdjustThreadPriority(RENDER_THREAD_PRIORITY);
		}
		
		UE_LOGF(LogAppleARKit, Log, "Restoring thread priorities: Game Thread (%d), Render Thread (%d)", GAME_THREAD_PRIORITY, RENDER_THREAD_PRIORITY);
}
#endif
	
#endif
	
#if SUPPORTS_ARKIT_3_0
	[CoachingOverlay release];
	CoachingOverlay = nullptr;
#endif
	
	// Set running state
	bIsRunning = false;
	
	return true;
}

void FAppleARKitSystem::OrientationChanged(const int32 NewOrientationRaw)
{
	const EDeviceScreenOrientation NewOrientation = static_cast<EDeviceScreenOrientation>(NewOrientationRaw);
	SetDeviceOrientationAndDerivedTracking(NewOrientation);
}
						
void FAppleARKitSystem::SessionDidUpdateFrame_DelegateThread(TSharedPtr< FAppleARKitFrame, ESPMode::ThreadSafe > Frame)
{
#if STATS && SUPPORTS_ARKIT_1_0
	const auto DelayMS = ([[NSProcessInfo processInfo] systemUptime] - Frame->Timestamp) * 1000.0;
	SET_FLOAT_STAT(STAT_ARKitFrameToDelegateDelay, DelayMS);
#endif
	
	{
		auto UpdateFrameTask = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP( this, &FAppleARKitSystem::SessionDidUpdateFrame_Internal, Frame.ToSharedRef() );
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(UpdateFrameTask, GET_STATID(STAT_FAppleARKitSystem_SessionUpdateFrame), nullptr, ENamedThreads::GameThread);
	}
	{
		UpdateARKitPerfStats();
#if SUPPORTS_ARKIT_1_0
		if (GetMutableDefault<UAppleARKitSettings>()->ShouldWriteCameraImagePerFrame())
		{
			WriteCameraImageToDisk(Frame->CameraImage);
		}
		
		if (CameraImage)
		{
			CameraImage->UpdateFrame(*Frame.Get());
		}
#endif
	}
}
			
void FAppleARKitSystem::SessionDidFailWithError_DelegateThread(const FString& Error)
{
	UE_LOGF(LogAppleARKit, Warning, "Session failed with error: %ls", *Error);
}

#if SUPPORTS_ARKIT_1_0

TArray<int32> FAppleARKitAnchorData::FaceIndices;
bool FAppleARKitAnchorData::bGenerateGeometry = false;
TSharedPtr<FARPose3D> FAppleARKitAnchorData::BodyRefPose = TSharedPtr<FARPose3D>();

static TSharedPtr<FAppleARKitAnchorData> MakePlaneAnchor(ARPlaneAnchor* PlaneAnchor)
{
	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(
		FAppleARKitConversion::ToFGuid(PlaneAnchor.identifier),
		FAppleARKitConversion::ToFTransform(PlaneAnchor.transform),
		FAppleARKitConversion::ToFVector(PlaneAnchor.center),
		// @todo use World Settings WorldToMetersScale
		0.5f * FAppleARKitConversion::ToFVector(PlaneAnchor.extent).GetAbs(),
		FAppleARKitConversion::ToEARPlaneOrientation(PlaneAnchor.alignment)
	);
	
#if SUPPORTS_ARKIT_1_5
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		const int32 NumBoundaryVerts = PlaneAnchor.geometry.boundaryVertexCount;
		NewAnchor->BoundaryVerts.Reset(NumBoundaryVerts);
		for (int32 i = 0; i < NumBoundaryVerts; ++i)
		{
			const vector_float3& Vert = PlaneAnchor.geometry.boundaryVertices[i];
			NewAnchor->BoundaryVerts.Add(FAppleARKitConversion::ToFVector(Vert));
		}
	}
#endif
	if (FAppleARKitAnchorData::bGenerateGeometry)
	{
		if (NewAnchor->BoundaryVerts.Num())
		{
			// Use the boundary verts if possible
			NewAnchor->Vertices = NewAnchor->BoundaryVerts;
			const auto NumPolygons = NewAnchor->BoundaryVerts.Num();
			NewAnchor->Indices.Reset(3 * NumPolygons);
			for (auto Index = 0; Index < NumPolygons; ++Index)
			{
				NewAnchor->Indices.Add(0);
				NewAnchor->Indices.Add((Index + 1) % NumPolygons);
				NewAnchor->Indices.Add((Index + 2) % NumPolygons);
			}
		}
		else
		{
			// Generate the mesh from the plane
			NewAnchor->Vertices.Reset(4);
			NewAnchor->Vertices.Add(NewAnchor->Center + NewAnchor->Extent);
			NewAnchor->Vertices.Add(NewAnchor->Center + FVector(NewAnchor->Extent.X, -NewAnchor->Extent.Y, NewAnchor->Extent.Z));
			NewAnchor->Vertices.Add(NewAnchor->Center + FVector(-NewAnchor->Extent.X, -NewAnchor->Extent.Y, NewAnchor->Extent.Z));
			NewAnchor->Vertices.Add(NewAnchor->Center + FVector(-NewAnchor->Extent.X, NewAnchor->Extent.Y, NewAnchor->Extent.Z));

			// Two triangles
			NewAnchor->Indices.Reset(6);
			NewAnchor->Indices.Add(0);
			NewAnchor->Indices.Add(1);
			NewAnchor->Indices.Add(2);
			NewAnchor->Indices.Add(2);
			NewAnchor->Indices.Add(3);
			NewAnchor->Indices.Add(0);
		}
	}
	
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		NewAnchor->ObjectClassification = FAppleARKitConversion::ToEARObjectClassification(PlaneAnchor.classification);
	}
#else
	NewAnchor->ObjectClassification = EARObjectClassification::Unknown;
#endif
	
	return NewAnchor;
}

#if SUPPORTS_ARKIT_1_5
static TSharedPtr<FAppleARKitAnchorData> MakeImageAnchor(ARImageAnchor* ImageAnchor)
{
	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(
		FAppleARKitConversion::ToFGuid(ImageAnchor.identifier),
		FAppleARKitConversion::ToFTransform(ImageAnchor.transform),
		EAppleAnchorType::ImageAnchor,
		FString(ImageAnchor.referenceImage.name)
	);
	
	NewAnchor->bIsTracked = ImageAnchor.isTracked;
	
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAnchorData::bGenerateGeometry)
	{
		FVector Extent(ImageAnchor.referenceImage.physicalSize.width, ImageAnchor.referenceImage.physicalSize.height, 0.f);
		// Scale by half since this is an extent around the center (same as scale then divide by 2)
		Extent *= 50.f;
		// Generate the mesh from the reference image's sizes
		NewAnchor->Vertices.Reset(4);
		NewAnchor->Vertices.Add(Extent);
		NewAnchor->Vertices.Add(FVector(Extent.X, -Extent.Y, Extent.Z));
		NewAnchor->Vertices.Add(FVector(-Extent.X, -Extent.Y, Extent.Z));
		NewAnchor->Vertices.Add(FVector(-Extent.X, Extent.Y, Extent.Z));
		
		// Two triangles
		NewAnchor->Indices.Reset(6);
		NewAnchor->Indices.Add(0);
		NewAnchor->Indices.Add(1);
		NewAnchor->Indices.Add(2);
		NewAnchor->Indices.Add(2);
		NewAnchor->Indices.Add(3);
		NewAnchor->Indices.Add(0);
	}
#endif
	return NewAnchor;
}
#endif // SUPPORTS_ARKIT_1_5

#if SUPPORTS_ARKIT_2_0
static TSharedPtr<FAppleARKitAnchorData> MakeProbeAnchor(AREnvironmentProbeAnchor* ProbeAnchor)
{
	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(
		FAppleARKitConversion::ToFGuid(ProbeAnchor.identifier),
		FAppleARKitConversion::ToFTransform(ProbeAnchor.transform),
		0.5f * FAppleARKitConversion::ToFVector(ProbeAnchor.extent).GetAbs(),
		ProbeAnchor.environmentTexture
	);
	return NewAnchor;
}

static TSharedPtr<FAppleARKitAnchorData> MakeObjectAnchor(ARObjectAnchor* ObjectAnchor)
{
	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(
		  FAppleARKitConversion::ToFGuid(ObjectAnchor.identifier),
		  FAppleARKitConversion::ToFTransform(ObjectAnchor.transform),
		  EAppleAnchorType::ObjectAnchor,
		  FString(ObjectAnchor.referenceObject.name)
	  );
	return NewAnchor;
}
#endif // SUPPORTS_ARKIT_2_0

#if SUPPORTS_ARKIT_3_0
static TSharedPtr<FAppleARKitAnchorData> MakeBodyAnchor(ARBodyAnchor* BodyAnchor)
{
	if (!FAppleARKitAnchorData::BodyRefPose)
	{
		FAppleARKitAnchorData::BodyRefPose = MakeShared<FARPose3D>(FAppleARKitConversion::ToARPose3D(BodyAnchor.skeleton.definition.neutralBodySkeleton3D, false));
	}

	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(
		  FAppleARKitConversion::ToFGuid(BodyAnchor.identifier),
		  FAppleARKitConversion::ToFTransform(BodyAnchor.transform),
		  FAppleARKitConversion::ToARPose3D(BodyAnchor)
	  );
	
	NewAnchor->bIsTracked = BodyAnchor.isTracked;
	
	return NewAnchor;
}
#endif

#if SUPPORTS_ARKIT_3_5
static TSharedPtr<FAppleARKitAnchorData> MakeMeshAnchor(ARMeshAnchor* MeshAnchor)
{
	const auto Guid = FAppleARKitConversion::ToFGuid(MeshAnchor.identifier);
	auto MeshData = FARKitMeshData::CacheMeshData(Guid, MeshAnchor);
	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(Guid, FAppleARKitConversion::ToFTransform(MeshAnchor.transform), MeshData);
	return NewAnchor;
}
#endif

#if SUPPORTS_ARKIT_4_0
static TSharedPtr<FAppleARKitAnchorData> MakeGeoAnchor(ARGeoAnchor* GeoAnchor)
{
	auto NewAnchor = MakeShared<FAppleARKitAnchorData>(
		  FAppleARKitConversion::ToFGuid(GeoAnchor.identifier),
		  FAppleARKitConversion::ToFTransform(GeoAnchor.transform),
		  GeoAnchor.coordinate.longitude,
		  GeoAnchor.coordinate.latitude,
		  GeoAnchor.altitude,
		  FAppleARKitConversion::ToAltitudeSource(GeoAnchor.altitudeSource)
	  );
	
	NewAnchor->bIsTracked = GeoAnchor.isTracked;
	
	return NewAnchor;
}
#endif

static TSharedPtr<FAppleARKitAnchorData> MakeAnchorData( ARAnchor* Anchor, double Timestamp, uint32 FrameNumber )
{
	TSharedPtr<FAppleARKitAnchorData> NewAnchor;
	
	// Plane Anchor
	if ([Anchor isKindOfClass:[ARPlaneAnchor class]])
	{
		ARPlaneAnchor* PlaneAnchor = (ARPlaneAnchor*)Anchor;
		NewAnchor = MakePlaneAnchor(PlaneAnchor);
	}

#if SUPPORTS_ARKIT_1_5
	// Image Anchor
	else if (FAppleARKitAvailability::SupportsARKit15() && [Anchor isKindOfClass:[ARImageAnchor class]])
	{
		ARImageAnchor* ImageAnchor = (ARImageAnchor*)Anchor;
		NewAnchor = MakeImageAnchor(ImageAnchor);
	}
#endif

#if SUPPORTS_ARKIT_2_0
	// Probe Anchor
	else if (FAppleARKitAvailability::SupportsARKit20() && [Anchor isKindOfClass:[AREnvironmentProbeAnchor class]])
	{
		AREnvironmentProbeAnchor* ProbeAnchor = (AREnvironmentProbeAnchor*)Anchor;
		NewAnchor = MakeProbeAnchor(ProbeAnchor);
	}
	
	// Object Anchor
	else if (FAppleARKitAvailability::SupportsARKit20() && [Anchor isKindOfClass:[ARObjectAnchor class]])
	{
		ARObjectAnchor* ObjectAnchor = (ARObjectAnchor*)Anchor;
		NewAnchor = MakeObjectAnchor(ObjectAnchor);
	}
#endif

#if SUPPORTS_ARKIT_3_0
	// Body Anchor
	else if (FAppleARKitAvailability::SupportsARKit30() && [Anchor isKindOfClass:[ARBodyAnchor class]])
	{
		ARBodyAnchor* BodyAnchor = (ARBodyAnchor*)Anchor;
		NewAnchor = MakeBodyAnchor(BodyAnchor);
	}
#endif

#if SUPPORTS_ARKIT_3_5
	// Mesh Anchor
	else if (FAppleARKitAvailability::SupportsARKit35() && [Anchor isKindOfClass:[ARMeshAnchor class]])
	{
		ARMeshAnchor* MeshAnchor = (ARMeshAnchor*)Anchor;
		NewAnchor = MakeMeshAnchor(MeshAnchor);
	}
#endif
	
#if SUPPORTS_ARKIT_4_0
	// Geo Anchor
	else if (FAppleARKitAvailability::SupportsARKit40() && [Anchor isKindOfClass:[ARGeoAnchor class]])
	{
		ARGeoAnchor* GeoAnchor = (ARGeoAnchor*)Anchor;
		NewAnchor = MakeGeoAnchor(GeoAnchor);
	}
#endif
	
	else
	{
		NewAnchor = MakeShared<FAppleARKitAnchorData>(
			FAppleARKitConversion::ToFGuid(Anchor.identifier),
			FAppleARKitConversion::ToFTransform(Anchor.transform));
	}

	NewAnchor->Timestamp = Timestamp;
	NewAnchor->FrameNumber = FrameNumber;
	
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		NewAnchor->AnchorName = Anchor.name;
	}
#endif
	
	return NewAnchor;
}
#endif

bool FAppleARKitSystem::OnTryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InAnchorName, UARPin*& OutAnchor)
{
    OutAnchor = nullptr;
#if SUPPORTS_ARKIT_1_0
    if (!FAppleARKitAvailability::SupportsARKit10())
    {
        return false;
    }
    
    if (!Session ||
        !InNativeResource)
    {
        return false;
    }
    
    ARAnchor* Anchor = static_cast<ARAnchor*>(InNativeResource);
    FGuid AnchorGUID = FAppleARKitConversion::ToFGuid(Anchor.identifier);
    
    {
        FScopeLock ScopeLock(&AnchorsLock);
        if (!AllAnchors.Contains(AnchorGUID))
        {
            AllAnchors.Add(AnchorGUID, Anchor);
        }
    }
    
    if (!TrackedGeometryGroups.Contains(AnchorGUID))
    {
        double Timestamp = FPlatformTime::Seconds();
        uint32 FrameNumber = TimecodeProvider->GetTimecode().Frames;
        auto AnchorData = MakeAnchorData(Anchor, Timestamp, FrameNumber);
        if (!TryCreateTrackedGeometry(AnchorData.ToSharedRef()))
        {
            return false;
        }
    }
    
    UARTrackedGeometry* TrackedGeometry = TrackedGeometryGroups[AnchorGUID].TrackedGeometry;
    for (auto Pin : Pins)
    {
        if (Pin->GetTrackedGeometry() == TrackedGeometry)
        {
            OutAnchor = Pin;
            return true;
        }
    }
    
    UARPin* NewPin = NewObject<UARPin>();
    
    NewPin->InitARPin(GetARCompositionComponent().ToSharedRef(), nullptr, TrackedGeometry->GetLocalToTrackingTransform(), TrackedGeometry, FName{ InAnchorName });
    
    // Retaining the ARAnchor prior to caching it as a native resource
    [Anchor retain];
    NewPin->SetNativeResource(InNativeResource);
    
    Pins.Add(NewPin);
    OutAnchor = NewPin;
    return true;
#else
    UE_LOGF(LogAppleARKit, Error, "Failed to create or get ARPin for native resource, ARKit is not supported on the current device platform.");
    return false;
#endif
}

#if SUPPORTS_ARKIT_1_0
void FAppleARKitSystem::SessionDidAddAnchors_DelegateThread( NSArray<ARAnchor*>* anchors )
{
	// Add the anchors to the record
	{
		FScopeLock ScopeLock(&AnchorsLock);
		for (ARAnchor* Anchor : anchors)
		{
			const auto Guid = FAppleARKitConversion::ToFGuid(Anchor.identifier);
			if (!AllAnchors.Contains(Guid))
			{
				[Anchor retain];
				AllAnchors.Add(Guid, Anchor);
			}
		}
	}
	
	// If this object is valid, we are running a face session and need that code to process things
	if (FaceARSupport != nullptr)
	{
		const FRotator& AdjustBy = GetARCompositionComponent()->GetSessionConfig().GetWorldAlignment() == EARWorldAlignment::Camera ? DerivedTrackingToUnrealRotation : FRotator::ZeroRotator;
		const EARFaceTrackingUpdate UpdateSetting = GetARCompositionComponent()->GetSessionConfig().GetFaceTrackingUpdate();

		const TArray<TSharedPtr<FAppleARKitAnchorData>> AnchorList = FaceARSupport->MakeAnchorData(anchors, AdjustBy, UpdateSetting);
		for (TSharedPtr<FAppleARKitAnchorData> NewAnchorData : AnchorList)
		{
			auto AddAnchorTask = FSimpleDelegateGraphTask::FDelegate::CreateSP(this, &FAppleARKitSystem::SessionDidAddAnchors_Internal, NewAnchorData.ToSharedRef());
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(AddAnchorTask, GET_STATID(STAT_FAppleARKitSystem_SessionDidAddAnchors), nullptr, ENamedThreads::GameThread);
		}
		return;
	}

	// Make sure all anchors get the same timestamp and frame number
	double Timestamp = FPlatformTime::Seconds();
	uint32 FrameNumber = TimecodeProvider->GetTimecode().Frames;

	for (ARAnchor* anchor in anchors)
	{
		TSharedPtr<FAppleARKitAnchorData> NewAnchorData = MakeAnchorData(anchor, Timestamp, FrameNumber);
		if (ensure(NewAnchorData.IsValid()))
		{
			auto AddAnchorTask = FSimpleDelegateGraphTask::FDelegate::CreateSP(this, &FAppleARKitSystem::SessionDidAddAnchors_Internal, NewAnchorData.ToSharedRef());
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(AddAnchorTask, GET_STATID(STAT_FAppleARKitSystem_SessionDidAddAnchors), nullptr, ENamedThreads::GameThread);
		}
	}
}

void FAppleARKitSystem::SessionDidUpdateAnchors_DelegateThread( NSArray<ARAnchor*>* anchors )
{
	// If this object is valid, we are running a face session and need that code to process things
	if (FaceARSupport != nullptr)
	{
		double UpdateTimestamp = FPlatformTime::Seconds();
		const FRotator& AdjustBy = GetARCompositionComponent()->GetSessionConfig().GetWorldAlignment() == EARWorldAlignment::Camera ? DerivedTrackingToUnrealRotation : FRotator::ZeroRotator;
		const EARFaceTrackingUpdate UpdateSetting = GetARCompositionComponent()->GetSessionConfig().GetFaceTrackingUpdate();

		const TArray<TSharedPtr<FAppleARKitAnchorData>> AnchorList = FaceARSupport->MakeAnchorData(anchors, AdjustBy, UpdateSetting);
		for (TSharedPtr<FAppleARKitAnchorData> NewAnchorData : AnchorList)
		{
			auto UpdateAnchorTask = FSimpleDelegateGraphTask::FDelegate::CreateSP(this, &FAppleARKitSystem::SessionDidUpdateAnchors_Internal, NewAnchorData.ToSharedRef());
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(UpdateAnchorTask, GET_STATID(STAT_FAppleARKitSystem_SessionDidUpdateAnchors), nullptr, ENamedThreads::GameThread);
		}
		return;
	}

	// Make sure all anchors get the same timestamp and frame number
	double Timestamp = FPlatformTime::Seconds();
	uint32 FrameNumber = TimecodeProvider->GetTimecode().Frames;

	for (ARAnchor* anchor in anchors)
	{
		TSharedPtr<FAppleARKitAnchorData> NewAnchorData = MakeAnchorData(anchor, Timestamp, FrameNumber);
		if (ensure(NewAnchorData.IsValid()))
		{
			auto UpdateAnchorTask = FSimpleDelegateGraphTask::FDelegate::CreateSP(this, &FAppleARKitSystem::SessionDidUpdateAnchors_Internal, NewAnchorData.ToSharedRef());
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(UpdateAnchorTask, GET_STATID(STAT_FAppleARKitSystem_SessionDidUpdateAnchors), nullptr, ENamedThreads::GameThread);
		}
	}
}

void FAppleARKitSystem::SessionDidRemoveAnchors_DelegateThread( NSArray<ARAnchor*>* anchors )
{
	// Remove the anchors from the record
	{
		FScopeLock ScopeLock(&AnchorsLock);
		for (ARAnchor* Anchor : anchors)
		{
			const auto Guid = FAppleARKitConversion::ToFGuid(Anchor.identifier);
			if (auto Record = AllAnchors.Find(Guid))
			{
				ARAnchor* SavedAnchor = *Record;
				[SavedAnchor release];
				AllAnchors.Remove(Guid);
			}
		}
	}
	
	// Face AR Anchors are also removed this way, no need for special code since they are tracked geometry
	for (ARAnchor* anchor in anchors)
	{
		// Convert to FGuid
		const FGuid AnchorGuid = FAppleARKitConversion::ToFGuid( anchor.identifier );

		auto RemoveAnchorTask = FSimpleDelegateGraphTask::FDelegate::CreateSP(this, &FAppleARKitSystem::SessionDidRemoveAnchors_Internal, AnchorGuid);
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(RemoveAnchorTask, GET_STATID(STAT_FAppleARKitSystem_SessionDidRemoveAnchors), nullptr, ENamedThreads::GameThread);
	}
}

void FAppleARKitSystem::SessionDidAddAnchors_Internal( TSharedRef<FAppleARKitAnchorData> AnchorData )
{
	const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& ARComponent = GetARCompositionComponent();
	const UARSessionConfig& SessionConfig = ARComponent->GetSessionConfig();
	
	// If an anchor was created by pinning a component, it will already have an associated geometry
	if (TrackedGeometryGroups.Contains(AnchorData->AnchorGUID))
	{
		return;
	}
	
	// In case we have camera tracking turned off, we still need to update the frame
	if (!SessionConfig.ShouldEnableCameraTracking())
	{
		UpdateFrame();
	}
	
	// If this object is valid, we are running a face session and we need to publish LiveLink data on the game thread
	if (FaceARSupport != nullptr && AnchorData->AnchorType == EAppleAnchorType::FaceAnchor)
	{
		FaceARSupport->PublishLiveLinkData(GetSessionGuid(), AnchorData);
	}

	if (PoseTrackingARLiveLink != nullptr && AnchorData->AnchorType == EAppleAnchorType::PoseAnchor)
	{
		PoseTrackingARLiveLink->PublishLiveLinkData(AnchorData);
	}

	TryCreateTrackedGeometry(AnchorData);
}

void FAppleARKitSystem::SessionDidUpdateAnchors_Internal( TSharedRef<FAppleARKitAnchorData> AnchorData )
{
	double UpdateTimestamp = FPlatformTime::Seconds();
	
	const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& ARComponent = GetARCompositionComponent();
	const FTransform& AlignmentTransform = ARComponent->GetAlignmentTransform();
	const auto AnchorWorldTransform = AnchorData->Transform * AlignmentTransform * GetTrackingToWorldTransform();

	// In case we have camera tracking turned off, we still need to update the frame
	if (!ARComponent->GetSessionConfig().ShouldEnableCameraTracking())
	{
		UpdateFrame();
	}

	// If this object is valid, we are running a face session and we need to publish LiveLink data on the game thread
	if (FaceARSupport != nullptr && AnchorData->AnchorType == EAppleAnchorType::FaceAnchor)
	{
		FaceARSupport->PublishLiveLinkData(GetSessionGuid(), AnchorData);
	}

	if (PoseTrackingARLiveLink != nullptr && AnchorData->AnchorType == EAppleAnchorType::PoseAnchor)
	{
		PoseTrackingARLiveLink->PublishLiveLinkData(AnchorData);
	}

	FTrackedGeometryGroup* TrackedGeometryGroup = TrackedGeometryGroups.Find(AnchorData->AnchorGUID);
	if (TrackedGeometryGroup != nullptr)
	{
		UARTrackedGeometry* FoundGeometry = TrackedGeometryGroup->TrackedGeometry;
		TArray<UARPin*> PinsToUpdate = ARKitUtil::PinsFromGeometry(FoundGeometry, Pins);
		
		// We figure out the delta transform for the Anchor (aka. TrackedGeometry in ARKit) and apply that
		// delta to figure out the new ARPin transform.
		const FTransform Anchor_LocalToTrackingTransform_PreUpdate = FoundGeometry->GetLocalToTrackingTransform_NoAlignment();
		const FTransform& Anchor_LocalToTrackingTransform_PostUpdate = AnchorData->Transform;
		
		const FTransform AnchorDeltaTransform = Anchor_LocalToTrackingTransform_PreUpdate.GetRelativeTransform(Anchor_LocalToTrackingTransform_PostUpdate);
		
		for (UARPin* Pin : PinsToUpdate)
		{
			const FTransform Pin_LocalToTrackingTransform_PostUpdate = Pin->GetLocalToTrackingTransform_NoAlignment() * AnchorDeltaTransform;
			Pin->OnTransformUpdated(Pin_LocalToTrackingTransform_PostUpdate);
		}
		
		FoundGeometry->SetTrackingState(AnchorData->bIsTracked ? EARTrackingState::Tracking : EARTrackingState::NotTracking);
				
		switch (AnchorData->AnchorType)
		{
			case EAppleAnchorType::Anchor:
			{
				FoundGeometry->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform);
				break;
			}
			case EAppleAnchorType::PlaneAnchor:
			{
				if (UARPlaneGeometry* PlaneGeo = Cast<UARPlaneGeometry>(FoundGeometry))
				{
					PlaneGeo->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->Center, AnchorData->Extent, AnchorData->BoundaryVerts, nullptr);
					PlaneGeo->SetOrientation(AnchorData->Orientation);
					PlaneGeo->SetObjectClassification(AnchorData->ObjectClassification);
				}
				break;
			}
			case EAppleAnchorType::FaceAnchor:
			{
				if (UARFaceGeometry* FaceGeo = Cast<UARFaceGeometry>(FoundGeometry))
				{
					FaceGeo->UpdateFaceGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->BlendShapes, AnchorData->FaceVerts, AnchorData->FaceIndices, AnchorData->FaceUVData, AnchorData->LeftEyeTransform, AnchorData->RightEyeTransform, AnchorData->LookAtTarget);
				}
				break;
			}
            case EAppleAnchorType::ImageAnchor:
            {
				if (UARTrackedImage* ImageAnchor = Cast<UARTrackedImage>(FoundGeometry))
				{
					auto* CandidateImage = CandidateImages.Find(AnchorData->DetectedAnchorName);
					ensure(CandidateImage != nullptr);
					FVector2D PhysicalSize((*CandidateImage)->GetPhysicalWidth(), (*CandidateImage)->GetPhysicalHeight());
					ImageAnchor->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, PhysicalSize, *CandidateImage);
				}
                break;
            }
			case EAppleAnchorType::EnvironmentProbeAnchor:
			{
				if (UAppleARKitEnvironmentCaptureProbe* ProbeAnchor = Cast<UAppleARKitEnvironmentCaptureProbe>(FoundGeometry))
				{
					// NOTE: The metal texture will be a different texture every time the cubemap is updated which requires a render resource flush
					ProbeAnchor->UpdateEnvironmentCapture(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->Extent, AnchorData->ProbeTexture);
				}
				break;
			}
			case EAppleAnchorType::PoseAnchor:
			{
				if (UARTrackedPose* TrackedPose = Cast<UARTrackedPose>(FoundGeometry))
				{
					TrackedPose->UpdateTrackedPose(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform, AnchorData->TrackedPose);
				}
				break;
			}
			case EAppleAnchorType::MeshAnchor:
			{
				if (UARMeshGeometry* MeshGeometry = Cast<UARMeshGeometry>(FoundGeometry))
				{
					MeshGeometry->UpdateTrackedGeometry(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform);
					
					UMRMeshComponent* MRMesh = MeshGeometry->GetUnderlyingMesh();
					if (MRMesh && AnchorData->MeshData)
					{
						FARKitMeshData::UpdateMRMesh(AnchorWorldTransform, AnchorData->MeshData, MRMesh);
					}
				}
				break;
			}
			case EAppleAnchorType::GeoAnchor:
			{
				if (UARGeoAnchor* GeoAnchor = Cast<UARGeoAnchor>(FoundGeometry))
				{
					GeoAnchor->UpdateGeoAnchor(ARComponent.ToSharedRef(), AnchorData->FrameNumber, AnchorData->Timestamp, AnchorData->Transform, AlignmentTransform,
											   AnchorData->Longitude, AnchorData->Latitude, AnchorData->AltitudeMeters, AnchorData->AltitudeSource);
				}
				break;
			}
		}
		
		// Trigger the delegate so anyone listening can take action
		if (TrackedGeometryGroup->ARComponent)
		{
			TrackedGeometryGroup->ARComponent->Update(TrackedGeometryGroup->TrackedGeometry);
			TriggerOnTrackableUpdatedDelegates(FoundGeometry);
		}
	}
}

void FAppleARKitSystem::SessionDidRemoveAnchors_Internal( FGuid AnchorGuid )
{
	const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& ARComponent = GetARCompositionComponent();

	// In case we have camera tracking turned off, we still need to update the frame
	if (!ARComponent->GetSessionConfig().ShouldEnableCameraTracking())
	{
		UpdateFrame();
	}

	// Notify pin that it is being orphaned
	{
		FTrackedGeometryGroup* TrackedGeometryGroup = TrackedGeometryGroups.Find(AnchorGuid);
		// This no longer performs a FindChecked() because the act of discard on restart can cause this to be missing
		if (TrackedGeometryGroup != nullptr)
		{
			UARTrackedGeometry* TrackedGeometryBeingRemoved = TrackedGeometryGroup->TrackedGeometry;
			if (TrackedGeometryGroup->ARComponent)
		{
				TrackedGeometryGroup->ARComponent->Remove(TrackedGeometryBeingRemoved);
				AARActor::RequestDestroyARActor(TrackedGeometryGroup->ARActor);
			}

			TrackedGeometryBeingRemoved->UpdateTrackingState(EARTrackingState::StoppedTracking);
			
			TArray<UARPin*> ARPinsBeingOrphaned = ARKitUtil::PinsFromGeometry(TrackedGeometryBeingRemoved, Pins);
			for(UARPin* PinBeingOrphaned : ARPinsBeingOrphaned)
			{
				PinBeingOrphaned->OnTrackingStateChanged(EARTrackingState::StoppedTracking);
			}
			// Trigger the delegate so anyone listening can take action
			TriggerOnTrackableRemovedDelegates(TrackedGeometryBeingRemoved);
		}
	}
	
	TrackedGeometryGroups.Remove(AnchorGuid);
	FARKitMeshData::RemoveMeshData(AnchorGuid);
}

#endif

void FAppleARKitSystem::SessionDidUpdateFrame_Internal( TSharedRef< FAppleARKitFrame, ESPMode::ThreadSafe > Frame )
{
	LastReceivedFrame = Frame;
	UpdateFrame();
}

#if STATS
struct FARKitThreadTimes
{
	TArray<FString> ThreadNames;
	int32 LastTotal;
	int32 NewTotal;
	
	FARKitThreadTimes() :
		LastTotal(0)
		, NewTotal(0)
	{
		ThreadNames.Add(TEXT("com.apple.CoreMotion"));
		ThreadNames.Add(TEXT("com.apple.arkit"));
		ThreadNames.Add(TEXT("FilteringFrameDownsampleNodeWorkQueue"));
		ThreadNames.Add(TEXT("FeatureDetectorNodeWorkQueue"));
		ThreadNames.Add(TEXT("SurfaceDetectionNode"));
		ThreadNames.Add(TEXT("VIOEngineNode"));
		ThreadNames.Add(TEXT("ImageDetectionQueue"));
	}

	bool IsARKitThread(const FString& Name)
	{
		if (Name.Len() == 0)
		{
			return false;
		}
		
		for (int32 Index = 0; Index < ThreadNames.Num(); Index++)
		{
			if (Name.StartsWith(ThreadNames[Index]))
			{
				return true;
			}
		}
		return false;
	}
	
	void FrameReset()
	{
		LastTotal = NewTotal;
		NewTotal = 0;
	}
};
#endif

void FAppleARKitSystem::UpdateARKitPerfStats()
{
#if STATS && SUPPORTS_ARKIT_1_0
	static FARKitThreadTimes ARKitThreadTimes;

	SCOPE_CYCLE_COUNTER(STAT_FAppleARKitSystem_UpdateARKitPerf);
	ARKitThreadTimes.FrameReset();
	
	thread_array_t ThreadArray;
	mach_msg_type_number_t ThreadCount;
	if (task_threads(mach_task_self(), &ThreadArray, &ThreadCount) != KERN_SUCCESS)
	{
		return;
	}

	for (int32 Index = 0; Index < (int32)ThreadCount; Index++)
	{
		mach_msg_type_number_t ThreadInfoCount = THREAD_BASIC_INFO_COUNT;
		mach_msg_type_number_t ExtThreadInfoCount = THREAD_EXTENDED_INFO_COUNT;
		thread_info_data_t ThreadInfo;
		thread_extended_info_data_t ExtThreadInfo;
		// Get the basic thread info for this thread
		if (thread_info(ThreadArray[Index], THREAD_BASIC_INFO, (thread_info_t)ThreadInfo, &ThreadInfoCount) != KERN_SUCCESS)
		{
			continue;
		}
		// And the extended thread info for this thread
		if (thread_info(ThreadArray[Index], THREAD_EXTENDED_INFO, (thread_info_t)&ExtThreadInfo, &ExtThreadInfoCount) != KERN_SUCCESS)
		{
			continue;
		}
		thread_basic_info_t BasicInfo = (thread_basic_info_t)ThreadInfo;
		FString ThreadName(ExtThreadInfo.pth_name);
		if (ARKitThreadTimes.IsARKitThread(ThreadName))
		{
			// CPU usage is reported as a scaled number, so convert to %
			int32 ScaledPercent = FMath::RoundToInt((float)BasicInfo->cpu_usage / (float)TH_USAGE_SCALE * 100.f);
			ARKitThreadTimes.NewTotal += ScaledPercent;
		}
//		UE_LOGF(LogAppleARKit, Log, "Thread %ls used cpu (%d), seconds (%d), microseconds (%d)", *ThreadName, BasicInfo->cpu_usage, BasicInfo->user_time.seconds + BasicInfo->system_time.seconds, BasicInfo->user_time.microseconds + BasicInfo->system_time.microseconds);
	}
	vm_deallocate(mach_task_self(), (vm_offset_t)ThreadArray, ThreadCount * sizeof(thread_t));
	SET_DWORD_STAT(STAT_ARKitThreads, ARKitThreadTimes.NewTotal);
#endif
}

#if SUPPORTS_ARKIT_1_0
void FAppleARKitSystem::WriteCameraImageToDisk(CVPixelBufferRef PixelBuffer)
{
	CFRetain(PixelBuffer);
	int32 ImageQuality = GetMutableDefault<UAppleARKitSettings>()->GetWrittenCameraImageQuality();
	float ImageScale = GetMutableDefault<UAppleARKitSettings>()->GetWrittenCameraImageScale();
	ETextureRotationDirection ImageRotation = GetMutableDefault<UAppleARKitSettings>()->GetWrittenCameraImageRotation();
	FTimecode Timecode = TimecodeProvider->GetTimecode();
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [PixelBuffer, ImageQuality, ImageScale, ImageRotation, Timecode]()
	{
		CIImage* SourceImage = [[CIImage alloc] initWithCVPixelBuffer: PixelBuffer];
		TArray<uint8> JpegBytes;
		IAppleImageUtilsPlugin::Get().ConvertToJPEG(SourceImage, JpegBytes, ImageQuality, true, true, ImageScale, ImageRotation);
		[SourceImage release];
		// Build a unique file name
		FDateTime DateTime = FDateTime::UtcNow();
		static FString UserDir = FPlatformProcess::UserDir();
		const FString& FaceDir = GetMutableDefault<UAppleARKitSettings>()->GetFaceTrackingLogDir();
		const TCHAR* SubDir = FaceDir.Len() > 0 ? *FaceDir : TEXT("CameraImages");
		FString FileName = FString::Printf(TEXT("%s%s/Image_%d-%d-%d-%d-%d-%d-%d.jpeg"), *UserDir, SubDir,
			DateTime.GetYear(), DateTime.GetMonth(), DateTime.GetDay(), Timecode.Hours, Timecode.Minutes, Timecode.Seconds, Timecode.Frames);
		// Write the jpeg to disk
		if (!FFileHelper::SaveArrayToFile(JpegBytes, *FileName))
		{
			UE_LOGF(LogAppleARKit, Error, "Failed to save JPEG to file name '%ls'", *FileName);
		}
		CFRelease(PixelBuffer);
	});
}
#endif

bool FAppleARKitSystem::OnIsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const
{
	return FAppleARKitConversion::IsSessionTrackingFeatureSupported(SessionType, SessionTrackingFeature);
}

TArray<FARPose2D> FAppleARKitSystem::OnGetTracked2DPose() const
{
	if (GameThreadFrame && GameThreadFrame->Tracked2DPose.SkeletonDefinition.NumJoints > 0)
	{
		return { GameThreadFrame->Tracked2DPose };
	}
	
	return {};
}

bool FAppleARKitSystem::OnIsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) const
{
	return FAppleARKitConversion::IsSceneReconstructionSupported(SessionType, SceneReconstructionMethod);
}

bool FAppleARKitSystem::OnAddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName)
{
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20() && Session)
	{
		if (bDeletePointsWithSameName)
		{
			FScopeLock ScopeLock(&AnchorsLock);
			for (auto Itr : AllAnchors)
			{
				auto Anchor = Itr.Value;
				if (FString(Anchor.name) == PointName)
				{
					[Session removeAnchor: Anchor];
				}
			}
		}
		
		const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& ARComponent = GetARCompositionComponent();
		const FTransform& AlignmentTransform = ARComponent->GetAlignmentTransform();
		const auto TrackingToWorldTransform = GetTrackingToWorldTransform();
		const auto AnchorTransform = WorldTransform * TrackingToWorldTransform.Inverse() * AlignmentTransform.Inverse();
		auto Anchor = [[ARAnchor alloc] initWithName: PointName.GetNSString() transform: FAppleARKitConversion::ToARKitMatrix(AnchorTransform)];
		[Session addAnchor: Anchor];
		return true;
	}
#endif
	return false;
}

int32 FAppleARKitSystem::OnGetNumberOfTrackedFacesSupported() const
{
	if (auto Implementation = GetModularInterface<IAppleARKitFaceSupport>())
	{
		return Implementation->GetNumberOfTrackedFacesSupported();
	}
	return 0;
}

bool FAppleARKitSystem::OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const
{
	if (GameThreadFrame)
	{
		OutCameraIntrinsics.ImageResolution = { (int32)GameThreadFrame->Camera.ImageResolution.X, (int32)GameThreadFrame->Camera.ImageResolution.Y };
		OutCameraIntrinsics.FocalLength = GameThreadFrame->Camera.FocalLength;
		OutCameraIntrinsics.PrincipalPoint = GameThreadFrame->Camera.PrincipalPoint;
		return true;
	}
	return false;
}

bool FAppleARKitSystem::GetGuidForGeometry(UARTrackedGeometry* InGeometry, FGuid& OutGuid) const
{
	for (auto GeoIt = TrackedGeometryGroups.CreateConstIterator(); GeoIt; ++GeoIt)
	{
		const FTrackedGeometryGroup& TrackedGeometryGroup = GeoIt.Value();

		if (TrackedGeometryGroup.TrackedGeometry == InGeometry)
		{
			OutGuid = GeoIt.Key();
			return true;
		}
	}
	
	return false;
}

FGuid FAppleARKitSystem::GetSessionGuid() const
{
#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30() && Session)
	{
		return FAppleARKitConversion::ToFGuid(Session.identifier);
	}
#endif
	
	return {};
}

namespace AppleARKitSupport
{
	TSharedPtr<class FAppleARKitSystem, ESPMode::ThreadSafe> CreateAppleARKitSystem()
	{
#if SUPPORTS_ARKIT_1_0
		// Handle older iOS devices somehow calling this
		if (FAppleARKitAvailability::SupportsARKit10())
		{
			auto NewARKitSystem = MakeShared<FAppleARKitSystem, ESPMode::ThreadSafe>();
            return NewARKitSystem;
		}
#endif
		return TSharedPtr<class FAppleARKitSystem, ESPMode::ThreadSafe>();
	}
}

UTimecodeProvider* UAppleARKitSettings::GetTimecodeProvider()
{
	const FString& ProviderName = GetDefault<UAppleARKitSettings>()->ARKitTimecodeProvider;
	UTimecodeProvider* TimecodeProvider = FindObject<UTimecodeProvider>(GEngine, *ProviderName);
	if (TimecodeProvider == nullptr)
	{
		// Try to load the class that was requested
		UClass* Class = LoadClass<UTimecodeProvider>(nullptr, *ProviderName);
		if (Class != nullptr)
		{
			TimecodeProvider = NewObject<UTimecodeProvider>(GEngine, Class);
		}
	}
	// Create the default one if this failed for some reason
	if (TimecodeProvider == nullptr)
	{
		TimecodeProvider = NewObject<UTimecodeProvider>(GEngine, UAppleARKitTimecodeProvider::StaticClass());
	}
	return TimecodeProvider;
}

void UAppleARKitSettings::CreateFaceTrackingLogDir()
{
	const FString& FaceDir = GetMutableDefault<UAppleARKitSettings>()->GetFaceTrackingLogDir();
	const TCHAR* SubDir = FaceDir.Len() > 0 ? *FaceDir : TEXT("FaceTracking");
	const FString UserDir = FPlatformProcess::UserDir();
	if (!IFileManager::Get().DirectoryExists(*(UserDir / SubDir)))
	{
		IFileManager::Get().MakeDirectory(*(UserDir / SubDir));
	}
}

void UAppleARKitSettings::CreateImageLogDir()
{
	const FString& FaceDir = GetMutableDefault<UAppleARKitSettings>()->GetFaceTrackingLogDir();
	const TCHAR* SubDir = FaceDir.Len() > 0 ? *FaceDir : TEXT("CameraImages");
	const FString UserDir = FPlatformProcess::UserDir();
	if (!IFileManager::Get().DirectoryExists(*(UserDir / SubDir)))
	{
		IFileManager::Get().MakeDirectory(*(UserDir / SubDir));
	}
}

FString UAppleARKitSettings::GetFaceTrackingLogDir()
{
	FScopeLock ScopeLock(&CriticalSection);
	return FaceTrackingLogDir;
}

bool UAppleARKitSettings::IsLiveLinkEnabledForFaceTracking()
{
	FScopeLock ScopeLock(&CriticalSection);
	return LivelinkTrackingTypes.Contains(ELivelinkTrackingType::FaceTracking);
}

bool UAppleARKitSettings::IsLiveLinkEnabledForPoseTracking()
{
	FScopeLock ScopeLock(&CriticalSection);
	return LivelinkTrackingTypes.Contains(ELivelinkTrackingType::PoseTracking);
}

bool UAppleARKitSettings::IsFaceTrackingLoggingEnabled()
{
	FScopeLock ScopeLock(&CriticalSection);
	return bFaceTrackingLogData;
}

bool UAppleARKitSettings::ShouldFaceTrackingLogPerFrame()
{
	FScopeLock ScopeLock(&CriticalSection);
	return bFaceTrackingWriteEachFrame;
}

EARFaceTrackingFileWriterType UAppleARKitSettings::GetFaceTrackingFileWriterType()
{
	FScopeLock ScopeLock(&CriticalSection);
	return FaceTrackingFileWriterType;
}

bool UAppleARKitSettings::ShouldWriteCameraImagePerFrame()
{
	FScopeLock ScopeLock(&CriticalSection);
	return bShouldWriteCameraImagePerFrame;
}

float UAppleARKitSettings::GetWrittenCameraImageScale()
{
	FScopeLock ScopeLock(&CriticalSection);
	return WrittenCameraImageScale;
}

int32 UAppleARKitSettings::GetWrittenCameraImageQuality()
{
	FScopeLock ScopeLock(&CriticalSection);
	return WrittenCameraImageQuality;
}

ETextureRotationDirection UAppleARKitSettings::GetWrittenCameraImageRotation()
{
	FScopeLock ScopeLock(&CriticalSection);
	return WrittenCameraImageRotation;
}

int32 UAppleARKitSettings::GetLiveLinkPublishingPort()
{
	FScopeLock ScopeLock(&CriticalSection);
	return LiveLinkPublishingPort;
}

FName UAppleARKitSettings::GetFaceTrackingLiveLinkSubjectName()
{
	FScopeLock ScopeLock(&CriticalSection);
	return DefaultFaceTrackingLiveLinkSubjectName;
}

FName UAppleARKitSettings::GetPoseTrackingLiveLinkSubjectName()
{
	FScopeLock ScopeLock(&CriticalSection);
	return DefaultPoseTrackingLiveLinkSubjectName;
}

EARFaceTrackingDirection UAppleARKitSettings::GetFaceTrackingDirection()
{
	FScopeLock ScopeLock(&CriticalSection);
	return DefaultFaceTrackingDirection;
}

bool UAppleARKitSettings::ShouldAdjustThreadPriorities()
{
	FScopeLock ScopeLock(&CriticalSection);
	return bAdjustThreadPrioritiesDuringARSession;
}

int32 UAppleARKitSettings::GetGameThreadPriorityOverride()
{
	FScopeLock ScopeLock(&CriticalSection);
	return GameThreadPriorityOverride;
}

int32 UAppleARKitSettings::GetRenderThreadPriorityOverride()
{
	FScopeLock ScopeLock(&CriticalSection);
	return RenderThreadPriorityOverride;
}

bool UAppleARKitSettings::Exec_Runtime(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("ARKitSettings")))
	{
		FScopeLock ScopeLock(&CriticalSection);

		if (FParse::Command(&Cmd, TEXT("StartFileWriting")))
		{
			UAppleARKitSettings::CreateFaceTrackingLogDir();
			bFaceTrackingLogData = true;
			bShouldWriteCameraImagePerFrame = true;
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("StopFileWriting")))
		{
			bFaceTrackingLogData = false;
			bShouldWriteCameraImagePerFrame = false;
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("StartCameraFileWriting")))
		{
			bShouldWriteCameraImagePerFrame = true;
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("StopCameraFileWriting")))
		{
			bShouldWriteCameraImagePerFrame = false;
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("SavePerFrame")))
		{
			bFaceTrackingWriteEachFrame = true;
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("SaveOnDemand")))
		{
			bFaceTrackingWriteEachFrame = false;
			return true;
		}
		else if (FParse::Value(Cmd, TEXT("FaceLogDir="), FaceTrackingLogDir))
		{
			UAppleARKitSettings::CreateFaceTrackingLogDir();
			return true;
		}
		else if (FParse::Value(Cmd, TEXT("LiveLinkSubjectName="), DefaultFaceTrackingLiveLinkSubjectName))
		{
			return true;
		}
	}
	return false;
}


/** Used to run Exec commands */
static bool MeshARTestingExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bHandled = false;

	if (FParse::Command(&Cmd, TEXT("ARKIT")))
	{
		if (FParse::Command(&Cmd, TEXT("MRMESH")))
		{
			AAROriginActor* OriginActor = AAROriginActor::GetOriginActor();
			UMRMeshComponent* NewComp = NewObject<UMRMeshComponent>(OriginActor);
			NewComp->RegisterComponent();
			NewComp->SetUseWireframe(true);
			// Send a fake update to it
			FTransform Transform = FTransform::Identity;
			TArray<FVector> Vertices;
			TArray<MRMESH_INDEX_TYPE> Indices;

			Vertices.Reset(4);
			Vertices.Add(FVector(100.f, 100.f, 0.f));
			Vertices.Add(FVector(100.f, -100.f, 0.f));
			Vertices.Add(FVector(-100.f, -100.f, 0.f));
			Vertices.Add(FVector(-100.f, 100.f, 0.f));

			Indices.Reset(6);
			Indices.Add(0);
			Indices.Add(1);
			Indices.Add(2);
			Indices.Add(2);
			Indices.Add(3);
			Indices.Add(0);

			NewComp->UpdateMesh(Transform.GetLocation(), Transform.GetRotation(), Transform.GetScale3D(), Vertices, Indices);

			return true;
		}
	}

	return false;
}

FStaticSelfRegisteringExec MeshARTestingExecRegistration(MeshARTestingExec);

#if PLATFORM_IOS
	#pragma clang diagnostic pop
#endif
