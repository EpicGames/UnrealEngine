// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "XRTrackingSystemBase.h"
#include "ARSystemSupportBase.h"
#include "AppleARKitTextures.h"
#include "Kismet/BlueprintPlatformLibrary.h"
#include "AppleARKitFaceSupport.h"
#include "AppleARKitPoseTrackingLiveLink.h"
#include "ARActor.h"

// ARKit
#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
	#include "AppleARKitSessionDelegate.h"
	#include "ARKitCoachingOverlay.h"
#endif


DECLARE_STATS_GROUP(TEXT("ARKit"), STATGROUP_ARKIT, STATCAT_Advanced);

//
//  FAppleARKitSystem
//

struct FAppleARKitFrame;
struct FAppleARKitAnchorData;

class FAppleARKitSystem : public FARSystemSupportBase, public FXRTrackingSystemBase, public FGCObject, public TSharedFromThis<FAppleARKitSystem, ESPMode::ThreadSafe>
{
	friend class FAppleARKitXRCamera;
	
	
public:
	FAppleARKitSystem();
	~FAppleARKitSystem();
	
	//~ IXRTrackingSystem
	FName GetSystemName() const override;
	int32 GetXRSystemFlags() const override;
	bool GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition) override;
	FString GetVersionString() const override;
	bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type) override;
	void ResetOrientationAndPosition(float Yaw) override;
	bool IsHeadTrackingAllowed() const override;
	TSharedPtr<class IXRCamera, ESPMode::ThreadSafe> GetXRCamera(int32 DeviceId) override;
	float GetWorldToMetersScale() const override;
	void OnBeginRendering_GameThread(FSceneViewFamily& InViewFamily) override;
	bool OnStartGameFrame(FWorldContext& WorldContext) override;
	//~ IXRTrackingSystem

	void* GetARSessionRawPointer() override;
	void* GetGameThreadARFrameRawPointer() override;
	
	/** So the module can shut down the ar services cleanly */
	void Shutdown();
	
	bool GetGuidForGeometry(UARTrackedGeometry* InGeometry, FGuid& OutGuid) const;

private:
	//~ FGCObject
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override { return "AppleARKitSystem"; }
	//~ FGCObject
protected:
	//~IARSystemSupport
	/** Returns true/false based on whether AR features are available */
	virtual bool IsARAvailable() const override;
	virtual void OnARSystemInitialized() override;
	virtual EARTrackingQuality OnGetTrackingQuality() const override;
	virtual EARTrackingQualityReason OnGetTrackingQualityReason() const override;
	virtual void OnStartARSession(UARSessionConfig* SessionConfig) override;
	virtual void OnPauseARSession() override;
	virtual void OnStopARSession() override;
	virtual FARSessionStatus OnGetARSessionStatus() const override;
	virtual void OnSetAlignmentTransform(const FTransform& InAlignmentTransform) override;
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects( const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels ) override;
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels) override;
	virtual TArray<UARTrackedGeometry*> OnGetAllTrackedGeometries() const override;
	virtual TArray<UARPin*> OnGetAllPins() const override;
	virtual bool OnIsTrackingTypeSupported(EARSessionType SessionType) const override;
	virtual UARLightEstimate* OnGetCurrentLightEstimate() const override;
	virtual UARPin* FindARPinByComponent(const USceneComponent* Component) const override;
	virtual UARPin* OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None) override;
	virtual void OnRemovePin(UARPin* PinToRemove) override;
    virtual bool OnTryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InAnchorName, UARPin*& OutAnchor) override;
	virtual UARTexture* OnGetARTexture(EARTextureType TextureType) const override;
	virtual bool OnAddManualEnvironmentCaptureProbe(FVector Location, FVector Extent) override;
	virtual TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> OnGetCandidateObject(FVector Location, FVector Extent) const override;
	virtual TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> OnSaveWorld() const override;
	virtual EARWorldMappingState OnGetWorldMappingStatus() const override;
	virtual TArray<FARVideoFormat> OnGetSupportedVideoFormats(EARSessionType SessionType) const override;
	virtual TArray<FVector> OnGetPointCloud() const override;
	virtual bool OnAddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth) override { return true; };
	
	virtual bool OnIsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const override;
	virtual TArray<FARPose2D> OnGetTracked2DPose() const override;
	virtual bool OnIsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) const override;
	virtual bool OnAddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName) override;
	virtual int32 OnGetNumberOfTrackedFacesSupported() const override;
	bool OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const override;
	//~IARSystemSupport

private:
	bool Run(UARSessionConfig* SessionConfig);
	bool IsRunning() const;
	bool Pause();
	void OrientationChanged(const int32 NewOrientation);
	void UpdateFrame();
	void CalcTrackingToWorldRotation();
#if SUPPORTS_ARKIT_1_0
	/** Asynchronously writes a JPEG to disk */
	void WriteCameraImageToDisk(CVPixelBufferRef PixelBuffer);
#endif
	class FAppleARKitXRCamera* GetARKitXRCamera();
	
public:
	// Session delegate callbacks
	void SessionDidUpdateFrame_DelegateThread( TSharedPtr< FAppleARKitFrame, ESPMode::ThreadSafe > Frame );
	void SessionDidFailWithError_DelegateThread( const FString& Error );
#if SUPPORTS_ARKIT_1_0
	void SessionDidAddAnchors_DelegateThread( NSArray<ARAnchor*>* anchors );
	void SessionDidUpdateAnchors_DelegateThread( NSArray<ARAnchor*>* anchors );
	void SessionDidRemoveAnchors_DelegateThread( NSArray<ARAnchor*>* anchors );
private:
	void SessionDidAddAnchors_Internal( TSharedRef<FAppleARKitAnchorData> AnchorData );
	void SessionDidUpdateAnchors_Internal( TSharedRef<FAppleARKitAnchorData> AnchorData );
	void SessionDidRemoveAnchors_Internal( FGuid AnchorGuid );
#endif
	void SessionDidUpdateFrame_Internal( TSharedRef< FAppleARKitFrame, ESPMode::ThreadSafe > Frame );
	/** Removes all tracked geometries, marking them as not tracked and sending the delegate event */
	void ClearTrackedGeometries();
	
public:
	
private:
	
	bool bIsRunning = false;
	
	void SetDeviceOrientationAndDerivedTracking(EDeviceScreenOrientation InOrientation);

	/** Creates or clears the face ar support object if face ar has been requested */
	void CheckForFaceARSupport(UARSessionConfig* InSessionConfig);

	/** Creates or clears the pose tracking ar support object if face ar has been requested */
	void CheckForPoseTrackingARLiveLink(UARSessionConfig* InSessionConfig);
	
	/** Updates the ARKit perf counters */
	void UpdateARKitPerfStats();

	/** Inits the textures and sets the texture on the overlay */
	void SetupCameraTextures();

    UARTrackedGeometry* TryCreateTrackedGeometry(TSharedRef<FAppleARKitAnchorData> AnchorData);
    
	void OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID);
	
	FGuid GetSessionGuid() const;


	/** The orientation of the device; see EDeviceScreenOrientation */
	EDeviceScreenOrientation DeviceOrientation;
	
	/** A rotation from ARKit TrackingSpace to Unreal Space. It is re-derived based on other parameters; users should not set it directly. */
	FRotator DerivedTrackingToUnrealRotation;

#if SUPPORTS_ARKIT_1_0

	// ARKit Session
	ARSession* Session = nullptr;
	
	// ARKit Session Delegate
	FAppleARKitSessionDelegate* Delegate = nullptr;

	/** The Metal texture cache for unbuffered texture uploads. */
	CVMetalTextureCacheRef MetalTextureCache = nullptr;
	
	/** Cache of images that we've converted previously to prevent repeated conversion */
	TMap< FString, CGImage* > ConvertedCandidateImages;
	
	// All the anchors added from the current session
	TMap<FGuid, ARAnchor*> AllAnchors;
	
	FCriticalSection AnchorsLock;
#endif
	
#if SUPPORTS_ARKIT_3_0
	FARKitCoachingOverlay* CoachingOverlay = nullptr;
#endif
	
	//
	// PROPERTIES REPORTED TO FGCObject
	// ...
	TMap< FGuid, FTrackedGeometryGroup > TrackedGeometryGroups;
	TArray<TObjectPtr<UARPin>> Pins;
	TObjectPtr<UARLightEstimate> LightEstimate = nullptr;
	TObjectPtr<UAppleARKitCameraVideoTexture> CameraImage = nullptr;
	TObjectPtr<UAppleARKitTextureCameraDepth> CameraDepth = nullptr;
	TMap< FString, TObjectPtr<UARCandidateImage> > CandidateImages;
	TMap< FString, TObjectPtr<UARCandidateObject> > CandidateObjects;
	TObjectPtr<UAppleARKitTextureCameraImage> SceneDepthMap = nullptr;
	TObjectPtr<UAppleARKitTextureCameraImage> SceneDepthConfidenceMap = nullptr;
	// ...
	// PROPERTIES REPORTED TO FGCObject
	//
	
	// An int counter that provides a human-readable debug number for Tracked Geometries.
	uint32 LastTrackedGeometry_DebugId;

	//'threadsafe' sharedptrs merely guaranteee atomicity when adding/removing refs.  You can still have a race
	//with destruction and copying sharedptrs.
	FCriticalSection FrameLock;

	// Last frame grabbed & processed by via the ARKit session delegate
	TSharedPtr< FAppleARKitFrame, ESPMode::ThreadSafe > GameThreadFrame;
	TSharedPtr< FAppleARKitFrame, ESPMode::ThreadSafe > RenderThreadFrame;
	TSharedPtr< FAppleARKitFrame, ESPMode::ThreadSafe > LastReceivedFrame;

	// The object that is handling face support if present
	IAppleARKitFaceSupport* FaceARSupport = nullptr;

	// The object that is handling pose tracking livelink if present
	IAppleARKitPoseTrackingLiveLink* PoseTrackingARLiveLink;

	/** The time code provider to use when tagging time stamps */
	TObjectPtr<UTimecodeProvider> TimecodeProvider = nullptr;

	//for networked callbacks
	FDelegateHandle SpawnARActorDelegateHandle;
};


namespace AppleARKitSupport
{
	APPLEARKIT_API TSharedPtr<class FAppleARKitSystem, ESPMode::ThreadSafe> CreateAppleARKitSystem();
}
