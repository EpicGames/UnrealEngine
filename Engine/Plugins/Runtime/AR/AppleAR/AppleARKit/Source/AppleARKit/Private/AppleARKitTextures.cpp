// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitTextures.h"
#include "AppleARKitModule.h"
#include "ExternalTexture.h"
#include "RenderingThread.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "AppleARKitAvailability.h"
#include "RenderGraph.h"
#include "ExternalTextureGuid.h"
#include "RHIResources.h"
#include "AppleARKitFrame.h"
#include "HAL/RunnableThread.h"
#include "GlobalShader.h"
#include "TextureResource.h"
#include "Materials/MaterialRenderProxy.h"

#if PLATFORM_APPLE
	THIRD_PARTY_INCLUDES_START
	#include "MetalInclude.h"
	THIRD_PARTY_INCLUDES_END

	#include "IMetalDynamicRHI.h"
	#import <Metal/Metal.h>
	#import <MetalPerformanceShaders/MetalPerformanceShaders.h>
	#include "Apple/ScopeAutoreleasePool.h"

#endif

DECLARE_CYCLE_STAT(TEXT("Scale Image"), STAT_ScaleImage, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("Blur Image"), STAT_BlurImage, STATGROUP_ARKIT);

static bool InRenderThread()
{
	if (GIsThreadedRendering)
	{
		return IsInActualRenderingThread();
	}
	
	return false;
}

#if PLATFORM_APPLE
class FAppleImageFilter
{
public:
	FAppleImageFilter(id<MTLDevice> MetalDevice)
	{
		check(MetalDevice);
		CommandQueue = [MetalDevice newCommandQueue];
	}
	
	virtual ~FAppleImageFilter()
	{
		if (CommandQueue)
		{
			[CommandQueue release];
			CommandQueue = nullptr;
		}
		
		if (GaussianBlur)
		{
			[GaussianBlur release];
			GaussianBlur = nullptr;
		}
	}
	
	static CIImage* GetScaledImage(CIImage* InImage, float SizeScale)
	{
		SCOPE_CYCLE_COUNTER(STAT_ScaleImage);
		
		if (!InImage || SizeScale == 1.f)
		{
			return InImage;
		}
		
		const auto AspectRatio = 1.f;
		CIFilter* ScaleFilter = [CIFilter filterWithName: @"CILanczosScaleTransform"];
		[ScaleFilter setValue: InImage forKey: kCIInputImageKey];
		[ScaleFilter setValue: @(SizeScale) forKey: kCIInputScaleKey];
		[ScaleFilter setValue: @(AspectRatio) forKey: kCIInputAspectRatioKey];
		return ScaleFilter.outputImage;
	}
	
	id<MTLTexture> ApplyGaussianBlur(id<MTLTexture> InTexture, float Sigma, bool bWaitUntilCompleted)
	{
		SCOPE_CYCLE_COUNTER(STAT_BlurImage);
		
		if (!InTexture)
		{
			return nullptr;
		}
		
		if (GaussianBlur && GaussianBlur.sigma != Sigma)
		{
			[GaussianBlur release];
			GaussianBlur = nullptr;
		}
		
		if (!GaussianBlur)
		{
			GaussianBlur = [[MPSImageGaussianBlur alloc] initWithDevice: CommandQueue.device sigma: Sigma];
		}

		if (!GaussianBlur)
		{
			return nullptr;
		}
		
		auto ProcessedTexture = InTexture;
		id<MTLCommandBuffer> CommandBuffer = [CommandQueue commandBuffer];
		[GaussianBlur encodeToCommandBuffer: CommandBuffer inPlaceTexture: &ProcessedTexture fallbackCopyAllocator: nil];
		[CommandBuffer commit];
		if (bWaitUntilCompleted)
		{
			[CommandBuffer waitUntilCompleted];
		}
		return ProcessedTexture;
	}
	
private:
	id<MTLCommandQueue> CommandQueue = nullptr;
	MPSImageGaussianBlur* GaussianBlur = nullptr;
};
#endif // PLATFORM_APPLE

class FARKitTextureResource : public FTextureResource
{
public:
	FARKitTextureResource(UTexture* InOwner)
		: Owner(InOwner)
	{
		bSRGB = InOwner->SRGB;
	}
	
	virtual ~FARKitTextureResource()
	{
#if PLATFORM_APPLE
		if (ImageContext)
		{
			CFRelease(ImageContext);
			ImageContext = nullptr;
		}
		
		ImageFilter = nullptr;
#endif
	}
	
	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const override { return Size.X; }
	
	/** Returns the height of the texture in pixels. */
	virtual uint32 GetSizeY() const override { return Size.Y; }
	
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);

		// Default to an empty 1x1 texture if we don't have a camera image
		Size.X = Size.Y = 1;

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FARKitTextureResource"), Size, PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource);
		
		TextureRHI = RHICmdList.CreateTexture(Desc);
		
		OnTextureUpdated(RHICmdList);
	}
	
	virtual void ReleaseRHI() override
	{
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
		FTextureResource::ReleaseRHI();
	}
	
#if PLATFORM_APPLE
	void UpdateCameraImage(CVPixelBufferRef CameraImage, EPixelFormat PixelFormat, const CFStringRef ColorSpace, FImageBlurParams BlurParams)
	{
		if (!CameraImage)
		{
			return;
		}
		CFRetain(CameraImage);
		ENQUEUE_RENDER_COMMAND(UpdateCameraImage)
		([this, CameraImage, PixelFormat, ColorSpace, BlurParams](FRHICommandListImmediate& RHICmdList)
		{
			UpdateCameraImage_RenderThread(RHICmdList, CameraImage, PixelFormat, ColorSpace, BlurParams);
			CFRelease(CameraImage);
		});
	}
	
	void UpdateMetalTexture(id<MTLTexture> MetalTexture, EPixelFormat PixelFormat, const CFStringRef ColorSpace)
	{
		if (!MetalTexture)
		{
			return;
		}
		
		CFRetain(MetalTexture);
		ENQUEUE_RENDER_COMMAND(UpdateMetalTextureResource)
		([this, MetalTexture, PixelFormat, ColorSpace](FRHICommandListImmediate& RHICmdList)
		{
			UpdateMetalTexture_RenderThread(RHICmdList, MetalTexture, PixelFormat, ColorSpace);
			CFRelease(MetalTexture);
		});
	}
	
protected:
	void ConditionalRecreateTexture(FRHICommandListImmediate& RHICmdList, uint32 Width, uint32 Height, EPixelFormat PixelFormat)
	{
		if (Size.X != Width || Size.Y != Height)
		{
			Size.X = Width;
			Size.Y = Height;
			
			// Let go of the last texture
			RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, nullptr);
			TextureRHI.SafeRelease();
			
			// Create the target texture that we'll update into
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FARKitTextureResource"), Size, PixelFormat)
				.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

			TextureRHI = RHICmdList.CreateTexture(Desc);
		}
	}
	
	void UpdateCameraImage_RenderThread(FRHICommandListImmediate& RHICmdList, CVPixelBufferRef CameraImage, EPixelFormat PixelFormat, const CFStringRef ColorSpace, FImageBlurParams BlurParams)
	{
		SCOPED_AUTORELEASE_POOL;
		CGColorSpaceRef ColorSpaceRef = CGColorSpaceCreateWithName(ColorSpace);
		CIImage* Image = [[CIImage alloc] initWithCVPixelBuffer: CameraImage];
		
		// Textures always need to be rotated so to a sane orientation (and mirrored because of differing coord system)
		static const TMap<EDeviceScreenOrientation, CGImagePropertyOrientation> OrientationMapping =
		{
			{ EDeviceScreenOrientation::Portrait, kCGImagePropertyOrientationRightMirrored },
			{ EDeviceScreenOrientation::LandscapeLeft, kCGImagePropertyOrientationUpMirrored },
			{ EDeviceScreenOrientation::PortraitUpsideDown, kCGImagePropertyOrientationLeftMirrored },
			{ EDeviceScreenOrientation::LandscapeRight, kCGImagePropertyOrientationDownMirrored },
		};
		
		auto ImageOrientation = kCGImagePropertyOrientationUp;
		if (auto Record = OrientationMapping.Find(FPlatformMisc::GetDeviceOrientation()))
		{
			ImageOrientation = *Record;
		}
		
		CIImage* RotatedImage = [Image imageByApplyingOrientation: ImageOrientation];
		
		if (BlurParams.GaussianBlurSigma > 0.f)
		{
			RotatedImage = FAppleImageFilter::GetScaledImage(RotatedImage, BlurParams.ImageSizeScale);
		}
		
		// Get the sizes from the rotated image
		CGRect ImageExtent = RotatedImage.extent;
		
		// Don't reallocate the texture if the sizes match
		ConditionalRecreateTexture(RHICmdList, ImageExtent.size.width, ImageExtent.size.height, PixelFormat);
		
		// Get the underlying metal texture so we can render to it
		id<MTLTexture> UnderlyingMetalTexture = (id<MTLTexture>)TextureRHI->GetNativeResource();
		
		[GetImageContext() render: RotatedImage toMTLTexture: UnderlyingMetalTexture commandBuffer: nil bounds: ImageExtent colorSpace: ColorSpaceRef];
		
		// Now that the conversion is done, we can get rid of our refs
		[Image release];
		CGColorSpaceRelease(ColorSpaceRef);
		
		if (BlurParams.GaussianBlurSigma > 0.f)
		{
			if (!ImageFilter)
			{
				id<MTLDevice> MetalDevice = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
				ImageFilter = MakeShared<FAppleImageFilter, ESPMode::ThreadSafe>(MetalDevice);
			}
			
			auto ProcessedTexture = ImageFilter->ApplyGaussianBlur(UnderlyingMetalTexture, BlurParams.GaussianBlurSigma, false /* bWaitUntilCompleted */);
			ensure(ProcessedTexture == UnderlyingMetalTexture);
		}
		
		OnTextureUpdated(RHICmdList);
	}
	
	void UpdateMetalTexture_RenderThread(FRHICommandListImmediate& RHICmdList, id<MTLTexture> MetalTexture, EPixelFormat PixelFormat, const CFStringRef ColorSpace)
	{
		SCOPED_AUTORELEASE_POOL;
		
		CGColorSpaceRef ColorSpaceRef = CGColorSpaceCreateWithName(ColorSpace);
		CIImage* Image = [[CIImage alloc] initWithMTLTexture: MetalTexture options: nil];
		
		// Textures always need to be rotated so to a sane orientation (and mirrored because of differing coord system)
		static const TMap<EDeviceScreenOrientation, CGImagePropertyOrientation> OrientationMapping =
		{
			{ EDeviceScreenOrientation::Portrait, kCGImagePropertyOrientationLeft },
			{ EDeviceScreenOrientation::LandscapeLeft, kCGImagePropertyOrientationDown },
			{ EDeviceScreenOrientation::PortraitUpsideDown, kCGImagePropertyOrientationRight },
			{ EDeviceScreenOrientation::LandscapeRight, kCGImagePropertyOrientationUp },
		};
		
		auto ImageOrientation = kCGImagePropertyOrientationUp;
		if (auto Record = OrientationMapping.Find(FPlatformMisc::GetDeviceOrientation()))
		{
			ImageOrientation = *Record;
		}
		
		CIImage* RotatedImage = [Image imageByApplyingOrientation: ImageOrientation];
		
		// Get the sizes from the rotated image
		CGRect ImageExtent = RotatedImage.extent;
		
		FIntPoint DesiredSize(ImageExtent.size.width, ImageExtent.size.height);
		
		if (!TextureRHI || DesiredSize != Size)
		{
			ConditionalRecreateTexture(RHICmdList, DesiredSize.X, DesiredSize.Y, PixelFormat);
		}
		
		if (TextureRHI)
		{
			// Get the underlying metal texture so we can render to it
			id<MTLTexture> UnderlyingMetalTexture = (id<MTLTexture>)TextureRHI->GetNativeResource();

			[GetImageContext() render: RotatedImage toMTLTexture: UnderlyingMetalTexture commandBuffer: nil bounds: ImageExtent colorSpace: ColorSpaceRef];
		}
		
		// Now that the conversion is done, we can get rid of our refs
		[Image release];
		CGColorSpaceRelease(ColorSpaceRef);
		
		OnTextureUpdated(RHICmdList);
	}
	
	CIContext* GetImageContext()
	{
		if (!ImageContext)
		{
			ImageContext = [CIContext context];
			CFRetain(ImageContext);
		}
		return ImageContext;
	}
#endif // PLATFORM_APPLE
	
protected:
	void OnTextureUpdated(FRHICommandListBase& RHICmdList)
	{
		RHICmdList.BindDebugLabelName(TextureRHI, *Owner->GetName());
		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
	}
	
protected:
	UTexture* Owner = nullptr;
	
	/** The size we get from the incoming camera image */
	FIntPoint Size = { 0, 0 };
	
#if PLATFORM_APPLE
	/** The cached image context that's reused between frames */
	CIContext* ImageContext = nullptr;
	
	TSharedPtr<FAppleImageFilter, ESPMode::ThreadSafe> ImageFilter = nullptr;
#endif
};

UAppleARKitTextureCameraImage::UAppleARKitTextureCameraImage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExternalTextureGuid = FGuid::NewGuid();
	SRGB = false;
}

FTextureResource* UAppleARKitTextureCameraImage::CreateResource()
{
	return new FARKitTextureResource(this);
}

void UAppleARKitTextureCameraImage::BeginDestroy()
{
#if PLATFORM_APPLE
	if (CameraImage != nullptr)
	{
		CFRelease(CameraImage);
		CameraImage = nullptr;
	}
#endif
	Super::BeginDestroy();
}

#if PLATFORM_APPLE
void UAppleARKitTextureCameraImage::UpdateCameraImage(float InTimestamp, CVPixelBufferRef InCameraImage, EPixelFormat InPixelFormat, const CFStringRef ColorSpace, FImageBlurParams BlurParams)
{
	check(IsInGameThread());
	
	if (CameraImage)
	{
		CFRelease(CameraImage);
		CameraImage = nullptr;
	}
	
	CameraImage = InCameraImage;
	if (CameraImage)
	{
		CFRetain(CameraImage);
		Size.X = CVPixelBufferGetWidth(CameraImage);
		Size.Y = CVPixelBufferGetHeight(CameraImage);
	}
	
	Timestamp = InTimestamp;
	
	if (!GetResource())
	{
		UpdateResource();
	}
	
	if (auto MyResource = static_cast<FARKitTextureResource*>(GetResource()))
	{
		MyResource->UpdateCameraImage(CameraImage, InPixelFormat, ColorSpace, BlurParams);
	}
}
#endif

UAppleARKitTextureCameraDepth::UAppleARKitTextureCameraDepth(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExternalTextureGuid = FGuid::NewGuid();
}

FTextureResource* UAppleARKitTextureCameraDepth::CreateResource()
{
	// @todo joeg -- hook this up for rendering
	return nullptr;
}

void UAppleARKitTextureCameraDepth::BeginDestroy()
{
#if PLATFORM_APPLE
	if (CameraDepth != nullptr)
	{
		CFRelease(CameraDepth);
		CameraDepth = nullptr;
	}
#endif
	Super::BeginDestroy();
}

#if SUPPORTS_ARKIT_1_0

void UAppleARKitTextureCameraDepth::Init(float InTimestamp, AVDepthData* InCameraDepth)
{
// @todo joeg -- finish this
	Timestamp = InTimestamp;
}

#endif

UAppleARKitEnvironmentCaptureProbeTexture::UAppleARKitEnvironmentCaptureProbeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExternalTextureGuid = FGuid::NewGuid();
	SRGB = false;
}

#if PLATFORM_APPLE
void UAppleARKitEnvironmentCaptureProbeTexture::Init(float InTimestamp, id<MTLTexture> InEnvironmentTexture)
{
	if (GetResource() == nullptr)
	{
		UpdateResource();
	}
	
	// Do nothing if the textures are the same
	// They will change as the data comes in but the textures themselves may stay the same between updates
	if (MetalTexture == InEnvironmentTexture)
	{
		return;
	}
	
	// Handle the case where this UObject is being reused
	if (MetalTexture != nullptr)
	{
		CFRelease(MetalTexture);
		MetalTexture = nullptr;
	}
	
	if (InEnvironmentTexture != nullptr)
	{
		Timestamp = InTimestamp;
		MetalTexture = InEnvironmentTexture;
		CFRetain(MetalTexture);
		Size.X = MetalTexture.width;
		Size.Y = MetalTexture.height;
	}
	// Force an update to our external texture on the render thread
	if (GetResource() != nullptr)
	{
		ENQUEUE_RENDER_COMMAND(UpdateEnvironmentCapture)(
			[InResource = GetResource()](FRHICommandListImmediate& RHICmdList)
			{
				InResource->InitRHI(RHICmdList);
			});
	}
}

class FARMetalResource :
	public FTextureResource
{
public:
	FARMetalResource(UAppleARKitEnvironmentCaptureProbeTexture* InOwner)
		: Owner(InOwner)
	{
		bGreyScaleFormat = false;
		bSRGB = InOwner->SRGB;
	}
	
	virtual ~FARMetalResource()
	{
		if (ImageContext)
		{
			CFRelease(ImageContext);
			ImageContext = nullptr;
		}
	}
	
	/**
	 * Called when the resource is initialized. This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FString TextureName = Owner->GetName();

		id<MTLTexture> MetalTexture = Owner->GetMetalTexture();
		if (MetalTexture != nullptr)
		{
			Size.X = Size.Y = Owner->Size.X;

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::CreateCube(*TextureName, Size.X, PF_B8G8R8A8)
				.SetFlags(ETextureCreateFlags::SRGB);

			EnvCubemapTextureRHIRef = RHICmdList.CreateTexture(Desc);

			/**
			 * To map their texture faces into our space we need:
			 *	 +X	to +Y	Down Mirrored
			 *	 -X to -Y	Up Mirrored
			 *	 +Y to +Z	Left Mirrored
			 *	 -Y to -Z	Left Mirrored
			 *	 +Z to -X	Left Mirrored
			 *	 -Z to +X	Right Mirrored
			 */
			CopyCubeFace(MetalTexture, EnvCubemapTextureRHIRef, kCGImagePropertyOrientationDownMirrored, 0, 2);
			CopyCubeFace(MetalTexture, EnvCubemapTextureRHIRef, kCGImagePropertyOrientationUpMirrored, 1, 3);
			CopyCubeFace(MetalTexture, EnvCubemapTextureRHIRef, kCGImagePropertyOrientationLeftMirrored, 2, 4);
			CopyCubeFace(MetalTexture, EnvCubemapTextureRHIRef, kCGImagePropertyOrientationLeftMirrored, 3, 5);
			CopyCubeFace(MetalTexture, EnvCubemapTextureRHIRef, kCGImagePropertyOrientationLeftMirrored, 4, 1);
			CopyCubeFace(MetalTexture, EnvCubemapTextureRHIRef, kCGImagePropertyOrientationRightMirrored, 5, 0);
		}
		else
		{
			Size.X = Size.Y = 1;

			const FRHITextureCreateDesc Desc = FRHITextureCreateDesc::CreateCube(*TextureName, Size.X, PF_B8G8R8A8);

			// Start with a 1x1 texture
			EnvCubemapTextureRHIRef = RHICmdList.CreateTexture(Desc);
		}


		TextureRHI = EnvCubemapTextureRHIRef;
		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);

		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
	}

	void CopyCubeFace(id<MTLTexture> MetalTexture, FTextureRHIRef Cubemap, uint32 Rotation, int32 MetalCubeIndex, int32 OurCubeIndex)
	{
		// Rotate the image we need to get a view into the face as a new slice
		id<MTLTexture> CubeFaceMetalTexture = [MetalTexture newTextureViewWithPixelFormat: MTLPixelFormatBGRA8Unorm textureType: MTLTextureType2D levels: NSMakeRange(0, 1) slices: NSMakeRange(MetalCubeIndex, 1)];
		CIImage* CubefaceImage = [[CIImage alloc] initWithMTLTexture: CubeFaceMetalTexture options: nil];
		CIImage* RotatedCubefaceImage = [CubefaceImage imageByApplyingOrientation: Rotation];
		CIImage* ImageTransform = nullptr;
		if (Rotation != kCGImagePropertyOrientationUp)
		{
			ImageTransform = RotatedCubefaceImage;
		}
		else
		{
			// We don't need to rotate it so just use a copy instead
			ImageTransform = CubefaceImage;
		}

		// Make a new view into our texture and directly render to that to avoid the CPU copy
		id<MTLTexture> UnderlyingMetalTexture = (id<MTLTexture>)Cubemap->GetNativeResource();
		id<MTLTexture> OurCubeFaceMetalTexture = [UnderlyingMetalTexture newTextureViewWithPixelFormat: MTLPixelFormatBGRA8Unorm textureType: MTLTextureType2D levels: NSMakeRange(0, 1) slices: NSMakeRange(OurCubeIndex, 1)];

		if (!ImageContext)
		{
			ImageContext = [CIContext context];
			CFRetain(ImageContext);
		}
		[ImageContext render: RotatedCubefaceImage toMTLTexture: OurCubeFaceMetalTexture commandBuffer: nil bounds: CubefaceImage.extent colorSpace: CubefaceImage.colorSpace];

		[CubefaceImage release];
		[CubeFaceMetalTexture release];
		[OurCubeFaceMetalTexture release];
	}
	
	virtual void ReleaseRHI() override
	{
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
		EnvCubemapTextureRHIRef.SafeRelease();
		FTextureResource::ReleaseRHI();
		FExternalTextureRegistry::Get().UnregisterExternalTexture(Owner->ExternalTextureGuid);
	}
	
	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const override
	{
		return Size.X;
	}
	
	/** Returns the height of the texture in pixels. */
	virtual uint32 GetSizeY() const override
	{
		return Size.Y;
	}
	
private:
	FIntPoint Size;
	
	FTextureRHIRef EnvCubemapTextureRHIRef;
	
	const UAppleARKitEnvironmentCaptureProbeTexture* Owner;
	
	CIContext* ImageContext = nullptr;
};

#endif // PLATFORM_APPLE

FTextureResource* UAppleARKitEnvironmentCaptureProbeTexture::CreateResource()
{
#if PLATFORM_APPLE
	return new FARMetalResource(this);
#else
	return nullptr;
#endif
}

void UAppleARKitEnvironmentCaptureProbeTexture::BeginDestroy()
{
#if PLATFORM_APPLE
	if (MetalTexture != nullptr)
	{
		CFRelease(MetalTexture);
		MetalTexture = nullptr;
	}
#endif
	Super::BeginDestroy();
}

UAppleARKitOcclusionTexture::UAppleARKitOcclusionTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SRGB = false;
}

void UAppleARKitOcclusionTexture::BeginDestroy()
{
#if PLATFORM_APPLE
	if (MetalTexture)
	{
		CFRelease(MetalTexture);
		MetalTexture = nullptr;
	}
#endif
	
	Super::BeginDestroy();
}

#if PLATFORM_APPLE
void UAppleARKitOcclusionTexture::SetMetalTexture(float InTimestamp, id<MTLTexture> InMetalTexture, EPixelFormat PixelFormat, const CFStringRef ColorSpace)
{
	{
		FScopeLock ScopeLock(&MetalTextureLock);
		Timestamp = InTimestamp;
		
		if (MetalTexture != InMetalTexture)
		{
			if (MetalTexture)
			{
				CFRelease(MetalTexture);
				MetalTexture = nullptr;
			}
			
			MetalTexture = InMetalTexture;
			
			if (MetalTexture)
			{
				CFRetain(MetalTexture);
				Size = FVector2f(MetalTexture.width, MetalTexture.height);
			}
		}
	}
	
	if (GetResource() == nullptr)
	{
		UpdateResource();
	}
	
	if (auto MyResource = static_cast<FARKitTextureResource*>(GetResource()))
	{
		MyResource->UpdateMetalTexture(InMetalTexture, PixelFormat, ColorSpace);
	}
}

id<MTLTexture> UAppleARKitOcclusionTexture::GetMetalTexture() const
{
	FScopeLock ScopeLock(&MetalTextureLock);
	return MetalTexture;
}

FTextureResource* UAppleARKitOcclusionTexture::CreateResource()
{
	return new FARKitTextureResource(this);
}

#else // PLATFORM_APPLE

FTextureResource* UAppleARKitOcclusionTexture::CreateResource()
{
	return nullptr;
}

#endif // PLATFORM_APPLE


class FComputeShaderYCbCrToRGB : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FComputeShaderYCbCrToRGB);
	SHADER_USE_PARAMETER_STRUCT(FComputeShaderYCbCrToRGB, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWTexture2D<float4>, OutputTexture)
		SHADER_PARAMETER(FVector2f, OutputTextureSize)
		SHADER_PARAMETER(FVector2f, InputTextureYSize)
		SHADER_PARAMETER(FVector2f, InputTextureCbCrSize)
		SHADER_PARAMETER(int, DeviceOrientation)
		SHADER_PARAMETER_TEXTURE(Texture2D, InputTextureY)
		SHADER_PARAMETER_TEXTURE(Texture2D, InputTextureCbCr)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE"), FComputeShaderUtils::kGolden2DGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeShaderYCbCrToRGB, "/AppleARKit/Private/ColorSpaceConversion.usf", "YCbCrToLinearRGB", SF_Compute);

class FARKitCameraVideoResource : public FTextureResource
{
public:
	FARKitCameraVideoResource(UAppleARKitCameraVideoTexture* InOwner)
		: Owner(InOwner)
	{
		bSRGB = InOwner->SRGB;
	}

	virtual ~FARKitCameraVideoResource()
	{
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
		
		// Default to an empty 1x1 texture if we don't have a camera image
		Size.X = Size.Y = 1;
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("DecodedTextureRef"), Size, PF_B8G8R8A8)
				.SetFlags(ETextureCreateFlags::ShaderResource);

			FScopeLock ScopeLock(&DecodedTextureLock);
			DecodedTextureRef = RHICmdList.CreateTexture(Desc);
		}
		
		UpdateTextureRHI(RHICmdList);
	}
	
	virtual void ReleaseRHI() override
	{
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
		DecodedTextureRef.SafeRelease();
		FTextureResource::ReleaseRHI();
	}
	
	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const override
	{
		return Size.X;
	}

	/** Returns the height of the texture in pixels. */
	virtual uint32 GetSizeY() const override
	{
		return Size.Y;
	}
	
	void UpdateTextureRHI(FRHICommandListBase& RHICmdList)
	{
		TextureRHI = DecodedTextureRef;
		RHICmdList.BindDebugLabelName(TextureRHI, *Owner->GetName());
		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
	}
	
#if SUPPORTS_ARKIT_1_0
	void UpdateVideoTexture(FRHICommandListImmediate& RHICmdList, CVMetalTextureRef CapturedYImage, FIntPoint CapturedYImageSize, CVMetalTextureRef CapturedCbCrImage, FIntPoint CapturedCbCrImageSize, EDeviceScreenOrientation DeviceOrientation)
	{
		// When the device rotates, the rendering thread will be destroyed and in which case IsInRenderingThread returns true...
		// We need to make sure that we're actually in a proper rendering thread before continue, otherwise we need to bail out to avoid weird crashes
		if (!InRenderThread())
		{
			CFRelease(CapturedYImage);
			CapturedYImage = nullptr;
			CFRelease(CapturedCbCrImage);
			CapturedCbCrImage = nullptr;
			return;
		}
		
		{
			IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();
			
			VideoTextureY = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_B8G8R8A8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, CapturedYImage);
			CFRelease(CapturedYImage);
	
			VideoTextureCbCr = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_B8G8R8A8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, CapturedCbCrImage);
			CFRelease(CapturedCbCrImage);

			// todo: Add an update call to the registry instead of this unregister/re-register
			FExternalTextureRegistry::Get().UnregisterExternalTexture(ARKitPassthroughCameraExternalTextureYGuid);
			FExternalTextureRegistry::Get().UnregisterExternalTexture(ARKitPassthroughCameraExternalTextureCbCrGuid);

			FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap);
			FSamplerStateRHIRef SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);

			FExternalTextureRegistry::Get().RegisterExternalTexture(ARKitPassthroughCameraExternalTextureYGuid, VideoTextureY, SamplerStateRHI);
			FExternalTextureRegistry::Get().RegisterExternalTexture(ARKitPassthroughCameraExternalTextureCbCrGuid, VideoTextureCbCr, SamplerStateRHI);
			
			//Make sure AR camera pass through materials are updated properly
			FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();
			
			CapturedYImage = nullptr;
			CapturedCbCrImage = nullptr;
		}
		
		auto OutputSize = CapturedYImageSize;
		if (DeviceOrientation == EDeviceScreenOrientation::Portrait || DeviceOrientation == EDeviceScreenOrientation::PortraitUpsideDown)
		{
			// Swap the X/Y size in portrait mode as the image needs to be rotated
			OutputSize = { CapturedYImageSize.Y, CapturedYImageSize.X };
		}
		
		// Recreate the decoded texture and its UAV if needed
		if (!DecodedTextureRef || Size != OutputSize)
		{
			Size = OutputSize;
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("FARKitCameraVideoResource_DecodedTexture"), Size, PF_B8G8R8A8)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				FScopeLock ScopeLock(&DecodedTextureLock);
				DecodedTextureRef = RHICmdList.CreateTexture(Desc);
			}
			
			DecodedTextureUAV = nullptr;
		}
		
		if (!DecodedTextureUAV)
		{
			DecodedTextureUAV = RHICmdList.CreateUnorderedAccessView(DecodedTextureRef, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(DecodedTextureRef));
		}
		
		{
			// Use a compute shader to do the color space conversion
			FRDGBuilder GraphBuilder(RHICmdList);
			FComputeShaderYCbCrToRGB::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeShaderYCbCrToRGB::FParameters>();
			PassParameters->OutputTexture = DecodedTextureUAV;
			PassParameters->OutputTextureSize = FVector2f(Size);

			PassParameters->InputTextureY = VideoTextureY;
			PassParameters->InputTextureYSize = FVector2f(CapturedYImageSize);
			
			// This mapping must be the same as the comment above YCbCrToLinearRGB!
			static const TMap<EDeviceScreenOrientation, int> DeviceOrientationIds =
			{
				{ EDeviceScreenOrientation::Portrait			, 0 },
				{ EDeviceScreenOrientation::PortraitUpsideDown	, 1 },
				{ EDeviceScreenOrientation::LandscapeLeft		, 2 },
				{ EDeviceScreenOrientation::LandscapeRight		, 3 },
			};
			
			if (auto Record = DeviceOrientationIds.Find(DeviceOrientation))
			{
				PassParameters->DeviceOrientation = *Record;
			}
			else
			{
				PassParameters->DeviceOrientation = 3;
			}
			
			PassParameters->InputTextureCbCr = VideoTextureCbCr;
			PassParameters->InputTextureCbCrSize = FVector2f(CapturedCbCrImageSize);

			TShaderMapRef<FComputeShaderYCbCrToRGB> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
			
			const auto GroupSize = FComputeShaderUtils::kGolden2DGroupSize;
			check(GroupSize * GroupSize <= GetMaxWorkGroupInvocations());

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ARKit Video Texture Conversion"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(Size, GroupSize)
			);
			
			GraphBuilder.Execute();
		}
		
		UpdateTextureRHI(RHICmdList);
	}
	
	id<MTLTexture> GetMetalTexture()
	{
		FScopeLock ScopeLock(&DecodedTextureLock);
		if (DecodedTextureRef)
		{
			return (id<MTLTexture>)DecodedTextureRef->GetNativeResource();
		}
		
		return nullptr;
	}
#endif
	
private:
	/** The size we get from the incoming camera image */
	FIntPoint Size;

	/** The texture that we actually render with which is populated via the GPU conversion process */
	FTextureRHIRef DecodedTextureRef;
	
	FCriticalSection DecodedTextureLock;
	
	FUnorderedAccessViewRHIRef DecodedTextureUAV;
	
	FTextureRHIRef VideoTextureY;
	FTextureRHIRef VideoTextureCbCr;

	const UAppleARKitCameraVideoTexture* Owner = nullptr;
};


FTextureResource* UAppleARKitCameraVideoTexture::CreateResource()
{
	return new FARKitCameraVideoResource(this);
}

UAppleARKitCameraVideoTexture::UAppleARKitCameraVideoTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SRGB = false;
}

void UAppleARKitCameraVideoTexture::Init()
{
	if (!GetResource())
	{
		UpdateResource();
	}
}

void UAppleARKitCameraVideoTexture::UpdateFrame(const FAppleARKitFrame& InFrame)
{
#if SUPPORTS_ARKIT_1_0
    if ((InFrame.CapturedYImage && InFrame.CapturedCbCrImage) &&
        (InFrame.CapturedYImageSize.GetMin() > 0) &&
        (InFrame.CapturedCbCrImageSize.GetMin() > 0))
	{
		CVMetalTextureRef CapturedYImageCopy = InFrame.CapturedYImage;
		CVMetalTextureRef CapturedCbCrImageCopy = InFrame.CapturedCbCrImage;
		CFRetain(CapturedYImageCopy);
		CFRetain(CapturedCbCrImageCopy);
		const FIntPoint CapturedYImageSize = InFrame.CapturedYImageSize;
		const FIntPoint CapturedCbCrImageSize = InFrame.CapturedCbCrImageSize;
		const EDeviceScreenOrientation DeviceOrientation = FPlatformMisc::GetDeviceOrientation();

		Size = CapturedYImageSize;

		ENQUEUE_RENDER_COMMAND(UpdateVideoTexture)(
			[this, CapturedYImageCopy, CapturedCbCrImageCopy, CapturedYImageSize, CapturedCbCrImageSize, DeviceOrientation](FRHICommandListImmediate& RHICmdList)
		{
			FARKitCameraVideoResource* VideoResource = static_cast<FARKitCameraVideoResource*>(GetResource());
			if (VideoResource != nullptr)
			{	
				VideoResource->UpdateVideoTexture(RHICmdList, CapturedYImageCopy, CapturedYImageSize, CapturedCbCrImageCopy, CapturedCbCrImageSize, DeviceOrientation);
			}
		});
	}
    else
#endif
    {
        Size = FVector2f::ZeroVector;
    }
}

#if SUPPORTS_ARKIT_1_0
id<MTLTexture> UAppleARKitCameraVideoTexture::GetMetalTexture() const
{
	if (auto MyResource = (FARKitCameraVideoResource*)GetResource())
	{
		return MyResource->GetMetalTexture();
	}
	return nullptr;
}
#endif
