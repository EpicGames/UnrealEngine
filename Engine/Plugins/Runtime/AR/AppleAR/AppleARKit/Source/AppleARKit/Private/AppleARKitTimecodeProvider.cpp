// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitTimecodeProvider.h"
#include "HAL/PlatformTime.h"


UAppleARKitTimecodeProvider::UAppleARKitTimecodeProvider()
	: FrameRate(60, 1)
{
}

bool UAppleARKitTimecodeProvider::Initialize(class UEngine*)
{
	return true;
}

void UAppleARKitTimecodeProvider::Shutdown(class UEngine*)
{
}

FQualifiedFrameTime UAppleARKitTimecodeProvider::GetQualifiedFrameTime() const
{
	// We construct a timecode to not have subframe.
	FTimecode Timecode = FTimecode(FPlatformTime::Seconds(), FrameRate, true);
	return FQualifiedFrameTime(Timecode, FrameRate);
}

ETimecodeProviderSynchronizationState UAppleARKitTimecodeProvider::GetSynchronizationState() const
{
	return ETimecodeProviderSynchronizationState::Synchronized;
}

