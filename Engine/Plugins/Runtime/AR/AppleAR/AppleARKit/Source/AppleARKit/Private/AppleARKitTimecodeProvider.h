// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimecodeProvider.h"
#include "AppleARKitTimecodeProvider.generated.h"

/**
 * This class is an implementation of the ITimecodeProvider and is used to abstract
 * out the calculation of the frame & time for an update
 */
UCLASS()
class UAppleARKitTimecodeProvider :
	public UTimecodeProvider
{
	GENERATED_BODY()

public:
	UAppleARKitTimecodeProvider();

	// ~UTimecodeProvider
	virtual FQualifiedFrameTime GetQualifiedFrameTime() const override;
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override;
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;
	// ~UTimecodeProvider

private:
	/** The frame rate of updates is 60 hz */
	FFrameRate FrameRate;
};

