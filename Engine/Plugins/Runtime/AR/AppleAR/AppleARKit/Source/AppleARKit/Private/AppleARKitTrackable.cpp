// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitTrackable.h"
#include "AppleARKitTextures.h"

#if PLATFORM_IOS
#import <Metal/MTLTexture.h>
#endif

UAppleARKitEnvironmentCaptureProbe::UAppleARKitEnvironmentCaptureProbe()
	: Super()
	, ARKitEnvironmentTexture(nullptr)
{
#if SUPPORTS_ARKIT_2_0
	ARKitEnvironmentTexture = NewObject<UAppleARKitEnvironmentCaptureProbeTexture>();
	// Set the base class member since that's what gets used by the non-ARKit specific code
	EnvironmentCaptureTexture = ARKitEnvironmentTexture;
#endif
}

#if PLATFORM_IOS
void UAppleARKitEnvironmentCaptureProbe::UpdateEnvironmentCapture(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double InTimestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector InExtent, id<MTLTexture> InMetalTexture)
{
	Super::UpdateEnvironmentCapture(InTrackingSystem, FrameNumber, InTimestamp, InLocalToTrackingTransform, InAlignmentTransform, InExtent);
	
	check(ARKitEnvironmentTexture != nullptr);
	ARKitEnvironmentTexture->Init(InTimestamp, InMetalTexture);
}
#endif

