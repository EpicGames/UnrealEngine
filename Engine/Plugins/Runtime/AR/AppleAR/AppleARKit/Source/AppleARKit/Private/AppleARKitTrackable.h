// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitAvailability.h"
#include "ARTrackable.h"
#include "AppleARKitTrackable.generated.h"

#if PLATFORM_IOS

@protocol MTLTexture;

#endif

class UAppleARKitEnvironmentCaptureProbeTexture;

UCLASS(NotBlueprintType)
class APPLEARKIT_API UAppleARKitEnvironmentCaptureProbe :
	public UAREnvironmentCaptureProbe
{
	GENERATED_BODY()

public:
	UAppleARKitEnvironmentCaptureProbe();
	
#if PLATFORM_IOS
	/** Updates the current capture with the new metal texture. This will trigger a render resource update if the textures have changed */
	void UpdateEnvironmentCapture(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector InExtent, id<MTLTexture> InMetalTexture);
#endif

private:
	/** The cube map of the reflected environment */
	UPROPERTY()
	TObjectPtr<UAppleARKitEnvironmentCaptureProbeTexture> ARKitEnvironmentTexture;
};
