// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitVideoOverlay.h"
#include "AppleARKitFrame.h"
#include "AppleARKitSystem.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"
#include "ExternalTexture.h"
#include "Misc/Guid.h"
#include "ExternalTextureGuid.h"
#include "Containers/ResourceArray.h"
#include "MediaShaders.h"
#include "PipelineStateCache.h"
#include "RHIUtilities.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIResourceUtils.h"
#include "RHIStaticStates.h"
#include "EngineModule.h"
#include "SceneUtils.h"
#include "RendererInterface.h"
#include "ScreenRendering.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "CommonRenderResources.h"
#include "ARUtilitiesFunctionLibrary.h"
#include "UObject/Package.h"
#include "PostProcess/DrawRectangle.h"
#include "ScreenPass.h"
#include "PostProcess/PostProcessMaterialInputs.h"

#if SUPPORTS_ARKIT_1_0
	#include "IOSAppDelegate.h"
#endif

#if SUPPORTS_ARKIT_3_0
	#include "IMetalDynamicRHI.h"
#endif

DECLARE_FLOAT_COUNTER_STAT(TEXT("ARKit Frame to Texture Delay (ms)"), STAT_ARKitFrameToTextureDelay, STATGROUP_ARKIT);
DECLARE_FLOAT_COUNTER_STAT(TEXT("ARKit Frame to Render Delay (ms)"), STAT_ARKitFrameToRenderDelay, STATGROUP_ARKIT);
DECLARE_CYCLE_STAT(TEXT("Update Occlusion Textures"), STAT_UpdateOcclusionTextures, STATGROUP_ARKIT);

#define VERTEX_BUFFER_INDEX_LANDSCAPE 0
#define VERTEX_BUFFER_INDEX_PORTRAIT 1

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define ALLOWS_DEBUG_OVERLAY 1
#else
	#define ALLOWS_DEBUG_OVERLAY 0
#endif

static int32 GDebugOverlayMode = 0;
static FAutoConsoleVariableRef CVarDebugOverlayMode(
	TEXT("arkit.DebugOverlayMode"),
	GDebugOverlayMode,
	TEXT("The debug overlay mode for ARKit:\n")
	TEXT("0: Disabled (Default)\n")
	TEXT("1: Show the person segmentation matte texture\n")
    TEXT("2: Show the person segmentation depth texture\n")
	TEXT("3: Show the scene depth map texture\n")
	TEXT("4: Show the scene depth confidence texture\n")
    TEXT("5: Show coloration of the scene depth data\n")
	);

enum class EDebugOverlayMode : uint8
{
	None = 0,
	PersonSegmentationMatte,
	PersonSegmentationDepth,
	SceneDepthMap,
	SceneDepthConfidence,
	SceneDepthColoration,
};

const FString UARKitCameraOverlayMaterialLoader::OverlayMaterialPath(TEXT("/ARUtilities/Materials/M_PassthroughCamera.M_PassthroughCamera"));
const FString UARKitCameraOverlayMaterialLoader::DepthOcclusionOverlayMaterialPath(TEXT("/AppleARKit/MI_DepthOcclusionOverlay.MI_DepthOcclusionOverlay"));
const FString UARKitCameraOverlayMaterialLoader::MatteOcclusionOverlayMaterialPath(TEXT("/AppleARKit/MI_MatteOcclusionOverlay.MI_MatteOcclusionOverlay"));
const FString UARKitCameraOverlayMaterialLoader::SceneDepthOcclusionMaterialPath(TEXT("/AppleARKit/M_SceneDepthOcclusion.M_SceneDepthOcclusion"));
const FString UARKitCameraOverlayMaterialLoader::SceneDepthColorationMaterialPath(TEXT("/ARUtilities/Materials/M_DepthColoration.M_DepthColoration"));

FAppleARKitVideoOverlay::FAppleARKitVideoOverlay()
	: MID_CameraOverlay(nullptr)
{
	auto MaterialLoader = GetDefault<UARKitCameraOverlayMaterialLoader>();
	
	MID_CameraOverlay = UMaterialInstanceDynamic::Create(MaterialLoader->DefaultCameraOverlayMaterial, GetTransientPackage());
	check(MID_CameraOverlay);
	
	MID_DepthOcclusionOverlay = UMaterialInstanceDynamic::Create(MaterialLoader->DepthOcclusionOverlayMaterial, GetTransientPackage());
	check(MID_DepthOcclusionOverlay);
	
	MID_MatteOcclusionOverlay = UMaterialInstanceDynamic::Create(MaterialLoader->MatteOcclusionOverlayMaterial, GetTransientPackage());
	check(MID_MatteOcclusionOverlay);
	
	MID_SceneDepthOcclusion = UMaterialInstanceDynamic::Create(MaterialLoader->SceneDepthOcclusionMaterial, GetTransientPackage());
	check(MID_SceneDepthOcclusion);
	
	MID_SceneDepthColoration = UMaterialInstanceDynamic::Create(MaterialLoader->SceneDepthColorationMaterial, GetTransientPackage());
	check(MID_SceneDepthColoration);
	
	CVarDebugOverlayMode->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* CVar)
	{
		UpdateDebugOverlay();
	}));
}

FAppleARKitVideoOverlay::~FAppleARKitVideoOverlay()
{
#if SUPPORTS_ARKIT_3_0
	if (CommandQueue)
	{
		[CommandQueue release];
		CommandQueue = nullptr;
	}
	
	if (MatteGenerator)
	{
		[MatteGenerator release];
		MatteGenerator = nullptr;
	}
#endif
	
	// Remove the callback as we're no longer valid
	CVarDebugOverlayMode->SetOnChangedCallback({});
}

BEGIN_SHADER_PARAMETER_STRUCT(FARKitPostProcessMaterialParameters, )
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PostProcessOutput)
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureInput, PostProcessInput, [kPostProcessMaterialInputCountMax])
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureInput, PathTracingPostProcessInput, [kPathTracingPostProcessMaterialInputCountMax])
	SHADER_PARAMETER_SAMPLER(SamplerState, PostProcessInput_BilinearSampler)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

template <bool bIsMobileRenderer>
class TPostProcessMaterialShader : public FMaterialShader
{
public:
	using FParameters = FARKitPostProcessMaterialParameters;
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(TPostProcessMaterialShader, FMaterialShader);

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		if (bIsMobileRenderer)
		{
			return Parameters.MaterialParameters.MaterialDomain == MD_PostProcess && IsMobilePlatform(Parameters.Platform);
		}
		else
		{
			return Parameters.MaterialParameters.MaterialDomain == MD_PostProcess && !IsMobilePlatform(Parameters.Platform);
		}
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL"), 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL_BEFORE_TONEMAP"), (Parameters.MaterialParameters.BlendableLocation != BL_SceneColorAfterTonemapping) ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_PASSTHROUGH"), 1);

		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), bMobileForceDepthRead ? 1u : 0u);
	}
};

// We use something similar to the PostProcessMaterial to render the color camera overlay.
template <bool bIsMobileRenderer>
class TARKitCameraOverlayVS : public TPostProcessMaterialShader<bIsMobileRenderer>
{
public:
	using FMaterialShader::FPermutationDomain;
	DECLARE_SHADER_TYPE(TARKitCameraOverlayVS, Material);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		TPostProcessMaterialShader<bIsMobileRenderer>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_PASSTHROUGH"), 1);

		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), bMobileForceDepthRead ? 1u : 0u);
	}

	TARKitCameraOverlayVS() = default;
	TARKitCameraOverlayVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: TPostProcessMaterialShader<bIsMobileRenderer>(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, this, View);
		TPostProcessMaterialShader<bIsMobileRenderer>::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
	}
};

using FARKitCameraOverlayVS = TARKitCameraOverlayVS<false>;
using FARKitCameraOverlayMobileVS = TARKitCameraOverlayVS<true>;

IMPLEMENT_SHADER_TYPE(template<>, FARKitCameraOverlayVS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainVS_VideoOverlay"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>, FARKitCameraOverlayMobileVS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainVS"), SF_Vertex);

template <bool bIsMobileRenderer>
class TARKitCameraOverlayPS : public TPostProcessMaterialShader<bIsMobileRenderer>
{
public:
	using FMaterialShader::FPermutationDomain;
	DECLARE_SHADER_TYPE(TARKitCameraOverlayPS, Material);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		TPostProcessMaterialShader<bIsMobileRenderer>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OUTPUT_MOBILE_HDR"), IsMobileHDR() ? 1 : 0);
	}

	TARKitCameraOverlayPS() = default;
	TARKitCameraOverlayPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: TPostProcessMaterialShader<bIsMobileRenderer>(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy, const FMaterial& Material)
	{
		TPostProcessMaterialShader<bIsMobileRenderer>::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		TPostProcessMaterialShader<bIsMobileRenderer>::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}
};

using FARKitCameraOverlayPS = TARKitCameraOverlayPS<false>;
using FARKitCameraOverlayMobilePS = TARKitCameraOverlayPS<true>;

IMPLEMENT_SHADER_TYPE(template<>, FARKitCameraOverlayPS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainPS_VideoOverlay"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, FARKitCameraOverlayMobilePS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainPS"), SF_Pixel);

void FAppleARKitVideoOverlay::UpdateVideoTextures(const FAppleARKitFrame& Frame)
{
}

void FAppleARKitVideoOverlay::UpdateOcclusionTextures(const FAppleARKitFrame& Frame)
{
#if SUPPORTS_ARKIT_3_0
	SCOPE_CYCLE_COUNTER(STAT_UpdateOcclusionTextures);
	
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		ARFrame* NativeFrame = (ARFrame*)Frame.NativeFrame;
		if (OcclusionType == EARKitOcclusionType::PersonSegmentation && NativeFrame &&
			(NativeFrame.segmentationBuffer || NativeFrame.estimatedDepthData))
		{
            id<MTLDevice> Device = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
			check(Device);

			if (!MatteGenerator)
			{
				// TODO: add config variable for the matte resolution
				MatteGenerator = [[ARMatteGenerator alloc] initWithDevice: Device
														  matteResolution: ARMatteResolutionFull];
			}

			if (!CommandQueue)
			{
				CommandQueue = [Device newCommandQueue];
			}

			id<MTLCommandBuffer> CommandBuffer = [CommandQueue commandBuffer];
			id<MTLTexture> MatteTexture = nullptr;
			id<MTLTexture> DepthTexture = nullptr;
			
			MatteTexture = [MatteGenerator generateMatteFromFrame: NativeFrame commandBuffer: CommandBuffer];
			DepthTexture = [MatteGenerator generateDilatedDepthFromFrame: NativeFrame commandBuffer: CommandBuffer];
			
			if (MatteTexture && OcclusionMatteTexture)
			{
				// For some reason int8 texture needs to use the sRGB color space to ensure the values are intact after CIImage processing
				OcclusionMatteTexture->SetMetalTexture(Frame.Timestamp, MatteTexture, PF_G8, kCGColorSpaceSRGB);
			}

			if (DepthTexture && OcclusionDepthTexture)
			{
				OcclusionDepthTexture->SetMetalTexture(Frame.Timestamp, DepthTexture, PF_R16F, kCGColorSpaceGenericRGBLinear);
				bOcclusionDepthTextureRecentlyUpdated = true;
			}

			[CommandBuffer commit];
		}
	}
#endif
}

void FAppleARKitVideoOverlay::RenderVideoOverlayWithMaterial(FRHICommandList& RHICmdList, const FSceneView& InView, struct FAppleARKitFrame& Frame, UMaterialInstanceDynamic* RenderingOverlayMaterial, const bool bRenderingOcclusion)
{
	if (RenderingOverlayMaterial == nullptr || !RenderingOverlayMaterial->IsValidLowLevel())
	{
		return;
	}

#if STATS && PLATFORM_IOS
	// LastUpdateTimestamp has been removed
	//const auto DelayMS = ([[NSProcessInfo processInfo] systemUptime] - LastUpdateTimestamp) * 1000.0;
	//SET_FLOAT_STAT(STAT_ARKitFrameToRenderDelay, DelayMS);
#endif

	SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, RenderVideoOverlay_Occlusion ,  bRenderingOcclusion, TEXT("VideoOverlay (Occlusion)"));
	SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, RenderVideoOverlay_Background, !bRenderingOcclusion, TEXT("VideoOverlay (Background)"));

	if (!OverlayVertexBufferRHI)
	{
		// Setup vertex buffer
		const FFilterVertex Vertices[] =
		{
			{ FVector4f(0.0f, 1.0f, 0.0f, 1.0f), FVector2f(0.0f, 1.0f) },
			{ FVector4f(0.0f, 0.0f, 0.0f, 1.0f), FVector2f(0.0f, 0.0f) },
			{ FVector4f(1.0f, 1.0f, 0.0f, 1.0f), FVector2f(1.0f, 1.0f) },
			{ FVector4f(1.0f, 0.0f, 0.0f, 1.0f), FVector2f(1.0f, 0.0f) },
		};

		OverlayVertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("VideoOverlayVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Vertices));
		
		// Cache UVOffsets
		const FVector2D ViewSize(InView.UnconstrainedViewRect.Max.X, InView.UnconstrainedViewRect.Max.Y);
		const FVector2D CameraSize = Frame.Camera.ImageResolution;
		for (auto Index = 0; Index < UE_ARRAY_COUNT(UVOffsets); ++Index)
		{
			if (Index == VERTEX_BUFFER_INDEX_LANDSCAPE)
			{
				// Landscape
				const auto ViewSizeLandscape = ViewSize.X > ViewSize.Y ? ViewSize : FVector2D(ViewSize.Y, ViewSize.X);
				const auto TextureSizeLandscape = CameraSize.X > CameraSize.Y ? CameraSize : FVector2D(CameraSize.Y, CameraSize.X);
				UVOffsets[Index] = UARUtilitiesFunctionLibrary::GetUVOffset(ViewSizeLandscape, TextureSizeLandscape);
			}
			else
			{
				// Portrait
				const auto ViewSizePortrait = ViewSize.X < ViewSize.Y ? ViewSize : FVector2D(ViewSize.Y, ViewSize.X);
				const auto TextureSizePortrait = CameraSize.X < CameraSize.Y ? CameraSize : FVector2D(CameraSize.Y, CameraSize.X);
				UVOffsets[Index] = UARUtilitiesFunctionLibrary::GetUVOffset(ViewSizePortrait, TextureSizePortrait);
			}
		}
	}
	
	if (!IndexBufferRHI)
	{
		// Setup index buffer
		const uint16 Indices[] = { 0, 1, 2, 2, 1, 3 };

		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("VideoOverlayIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}

	const auto FeatureLevel = InView.GetFeatureLevel();
	IRendererModule& RendererModule = GetRendererModule();

	const FMaterialRenderProxy* MaterialProxy = RenderingOverlayMaterial->GetRenderProxy();
	const FMaterial& CameraMaterial = MaterialProxy->GetMaterialWithFallback(FeatureLevel, MaterialProxy);
	const FMaterialShaderMap* const MaterialShaderMap = CameraMaterial.GetRenderingThreadShaderMap();

	// If there are pending MI parameter updates, ensure these get processed
	if (MaterialProxy->UniformExpressionCache[FeatureLevel].CachedUniformExpressionShaderMap == nullptr)
	{
		FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions(RHICmdList);
	}
	
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	if (bRenderingOcclusion)
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	}
	else
	{
		// Disable the write mask for the alpha channel so that the scene depth info saved in it is retained
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
	}
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;

	const bool bIsMobileRenderer = FeatureLevel <= ERHIFeatureLevel::ES3_1;
	TShaderRef<FMaterialShader> VertexShader;
	TShaderRef<FMaterialShader> PixelShader;
	if (bIsMobileRenderer)
	{
		VertexShader = MaterialShaderMap->GetShader<FARKitCameraOverlayMobileVS>();
		check(VertexShader.IsValid());
		PixelShader = MaterialShaderMap->GetShader<FARKitCameraOverlayMobilePS>();
		check(PixelShader.IsValid());
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	}
	else
	{
		VertexShader = MaterialShaderMap->GetShader<FARKitCameraOverlayVS>();
		check(VertexShader.IsValid());
		PixelShader = MaterialShaderMap->GetShader<FARKitCameraOverlayPS>();
		check(PixelShader.IsValid());
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	}

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
	
	if (bIsMobileRenderer)
	{
		TShaderRef<FARKitCameraOverlayMobileVS> VertexShaderPtr(static_cast<FARKitCameraOverlayMobileVS*>(VertexShader.GetShader()), *MaterialShaderMap);
		TShaderRef<FARKitCameraOverlayMobilePS> PixelShaderPtr(static_cast<FARKitCameraOverlayMobilePS*>(PixelShader.GetShader()), *MaterialShaderMap);

		SetShaderParametersLegacyVS(RHICmdList, VertexShaderPtr, InView);
		SetShaderParametersLegacyPS(RHICmdList, PixelShaderPtr, InView, MaterialProxy, CameraMaterial);
	}
	else
	{
		TShaderRef<FARKitCameraOverlayVS> VertexShaderPtr(static_cast<FARKitCameraOverlayVS*>(VertexShader.GetShader()), *MaterialShaderMap);
		TShaderRef<FARKitCameraOverlayPS> PixelShaderPtr(static_cast<FARKitCameraOverlayPS*>(PixelShader.GetShader()), *MaterialShaderMap);

		SetShaderParametersLegacyVS(RHICmdList, VertexShaderPtr, InView);
		SetShaderParametersLegacyPS(RHICmdList, PixelShaderPtr, InView, MaterialProxy, CameraMaterial);
	}
	
	if (OverlayVertexBufferRHI && IndexBufferRHI)
	{
		RHICmdList.SetStreamSource(0, OverlayVertexBufferRHI, 0);
		RHICmdList.DrawIndexedPrimitive(
			IndexBufferRHI,
			/*BaseVertexIndex=*/ 0,
			/*MinIndex=*/ 0,
			/*NumVertices=*/ 4,
			/*StartIndex=*/ 0,
			/*NumPrimitives=*/ 2,
			/*NumInstances=*/ 1
		);
	}
}

void FAppleARKitVideoOverlay::RenderVideoOverlay_RenderThread(FRHICommandList& RHICmdList, const FSceneView& InView, FAppleARKitFrame& Frame, const float WorldToMeterScale)
{
	UpdateVideoTextures(Frame);
	UpdateOcclusionTextures(Frame);
	
	auto OverlayMaterial = MID_CameraOverlay;
	
#if ALLOWS_DEBUG_OVERLAY
	if (GDebugOverlayMode == (int32)EDebugOverlayMode::SceneDepthColoration && OcclusionType == EARKitOcclusionType::SceneDepth)
	{
		OverlayMaterial = MID_SceneDepthColoration;
	}
#endif
	
	RenderVideoOverlayWithMaterial(RHICmdList, InView, Frame, OverlayMaterial, false);
	
#if ALLOWS_DEBUG_OVERLAY
	if (GDebugOverlayMode)
	{
		// Do not draw the occlusion overlay in debug mode
		return;
	}
#endif
	
	if (OcclusionType == EARKitOcclusionType::PersonSegmentation)
	{
		UMaterialInstanceDynamic* OcclusionMaterial = bOcclusionDepthTextureRecentlyUpdated ? MID_DepthOcclusionOverlay : MID_MatteOcclusionOverlay;
		UARUtilitiesFunctionLibrary::UpdateWorldToMeterScale(OcclusionMaterial, WorldToMeterScale);
		RenderVideoOverlayWithMaterial(RHICmdList, InView, Frame, OcclusionMaterial, true);
		bOcclusionDepthTextureRecentlyUpdated = false;
	}
	else if (OcclusionType == EARKitOcclusionType::SceneDepth)
	{
		UARUtilitiesFunctionLibrary::UpdateWorldToMeterScale(MID_SceneDepthOcclusion, WorldToMeterScale);
		RenderVideoOverlayWithMaterial(RHICmdList, InView, Frame, MID_SceneDepthOcclusion, true);
	}
}

bool FAppleARKitVideoOverlay::GetPassthroughCameraUVs_RenderThread(TArray<FVector2D>& OutUVs, const EDeviceScreenOrientation DeviceOrientation)
{
	const auto bLandscape = (DeviceOrientation == EDeviceScreenOrientation::LandscapeLeft) || (DeviceOrientation == EDeviceScreenOrientation::LandscapeRight);
	const auto& UVOffset = bLandscape ? UVOffsets[VERTEX_BUFFER_INDEX_LANDSCAPE] : UVOffsets[VERTEX_BUFFER_INDEX_PORTRAIT];
	UARUtilitiesFunctionLibrary::GetPassthroughCameraUVs(OutUVs, UVOffset);
	return true;
}

void FAppleARKitVideoOverlay::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Add all the referenced materials
	Collector.AddReferencedObject(MID_CameraOverlay);
	Collector.AddReferencedObject(MID_DepthOcclusionOverlay);
	Collector.AddReferencedObject(MID_MatteOcclusionOverlay);
	Collector.AddReferencedObject(MID_SceneDepthOcclusion);
	Collector.AddReferencedObject(MID_SceneDepthColoration);
	
	// Add all the referenced textures
	Collector.AddReferencedObject(OcclusionMatteTexture);
	Collector.AddReferencedObject(OcclusionDepthTexture);
	Collector.AddReferencedObject(CameraTexture);
	Collector.AddReferencedObject(SceneDepthTexture);
	Collector.AddReferencedObject(SceneDepthConfidenceTexture);
}

void FAppleARKitVideoOverlay::SetOverlayTexture(UARTextureCameraImage* InCameraImage)
{
	CameraTexture = InCameraImage;
	UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(MID_CameraOverlay, InCameraImage);
	UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(MID_DepthOcclusionOverlay, InCameraImage);
	UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(MID_MatteOcclusionOverlay, InCameraImage);
	UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(MID_SceneDepthOcclusion, InCameraImage);
	
	UpdateDebugOverlay();
}

void FAppleARKitVideoOverlay::UpdateSceneDepthTextures(UTexture* InSceneDepthTexture, UTexture* InDepthConfidenceTexture)
{
	if (MID_SceneDepthOcclusion)
	{
		static const FName DepthConfidenceTextureParamName(TEXT("DepthConfidenceTexture"));
		
		UARUtilitiesFunctionLibrary::UpdateSceneDepthTexture(MID_SceneDepthOcclusion, InSceneDepthTexture);
		MID_SceneDepthOcclusion->SetTextureParameterValue(DepthConfidenceTextureParamName, InDepthConfidenceTexture);
	}
	
	if (MID_SceneDepthColoration)
	{
		UARUtilitiesFunctionLibrary::UpdateSceneDepthTexture(MID_SceneDepthColoration, InSceneDepthTexture);
	}
	
	SceneDepthTexture = InSceneDepthTexture;
	SceneDepthConfidenceTexture = InDepthConfidenceTexture;
	
	UpdateDebugOverlay();
}

void FAppleARKitVideoOverlay::SetOcclusionType(EARKitOcclusionType InOcclusionType)
{
#if SUPPORTS_ARKIT_3_0
	OcclusionType = InOcclusionType;
	
	if (OcclusionType == EARKitOcclusionType::PersonSegmentation)
	{
		OcclusionMatteTexture = UARTexture::CreateARTexture<UAppleARKitOcclusionTexture>(EARTextureType::PersonSegmentationImage);
		OcclusionMatteTexture->UpdateResource();
		
		OcclusionDepthTexture = UARTexture::CreateARTexture<UAppleARKitOcclusionTexture>(EARTextureType::PersonSegmentationDepth);
		OcclusionDepthTexture->UpdateResource();
	}
	else
	{
		OcclusionMatteTexture = OcclusionDepthTexture = nullptr;
	}
	
	static const FName MatteTextureParamName(TEXT("OcclusionMatteTexture"));
	static const FName DepthTextureParamName(TEXT("OcclusionDepthTexture"));
	
	MID_DepthOcclusionOverlay->SetTextureParameterValue(MatteTextureParamName, OcclusionMatteTexture);
	MID_DepthOcclusionOverlay->SetTextureParameterValue(DepthTextureParamName, OcclusionDepthTexture);
	
	MID_MatteOcclusionOverlay->SetTextureParameterValue(MatteTextureParamName, OcclusionMatteTexture);
	MID_MatteOcclusionOverlay->SetTextureParameterValue(DepthTextureParamName, OcclusionDepthTexture);
	
	UpdateDebugOverlay();
#endif
}

void FAppleARKitVideoOverlay::UpdateDebugOverlay()
{
#if ALLOWS_DEBUG_OVERLAY
	if (MID_CameraOverlay)
	{
		UTexture* OverlayTexture = nullptr;
		float ColorScale = 1.f;
		switch (GDebugOverlayMode)
		{
			case (int32)EDebugOverlayMode::None:
				OverlayTexture = CameraTexture;
				break;
			
			case (int32)EDebugOverlayMode::PersonSegmentationMatte:
				OverlayTexture = OcclusionMatteTexture;
				break;
				
			case (int32)EDebugOverlayMode::PersonSegmentationDepth:
				OverlayTexture = OcclusionDepthTexture;
				break;
				
			case (int32)EDebugOverlayMode::SceneDepthMap:
				OverlayTexture = SceneDepthTexture;
				break;
				
			case (int32)EDebugOverlayMode::SceneDepthConfidence:
				OverlayTexture = SceneDepthConfidenceTexture;
				// Use the color scale to map ARConfidenceLevelHigh to 255
				// See https://developer.apple.com/documentation/arkit/arconfidencelevel
#if SUPPORTS_ARKIT_4_0
				ColorScale = 255.f / (float)ARConfidenceLevelHigh;
#endif
				break;
		}
		
		if (OverlayTexture)
		{
			UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(MID_CameraOverlay, OverlayTexture, ColorScale);
		}
	}
#endif
}
