// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "AppleARKitAvailability.h"

#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

#include "AppleARKitVideoOverlay.generated.h"

class FRHIBuffer;
class FRHICommandListImmediate;
class UARTextureCameraImage;
class UMaterialInstanceDynamic;
class UAppleARKitOcclusionTexture;
typedef TRefCountPtr<FRHIBuffer> FBufferRHIRef;

enum class EARKitOcclusionType : uint8
{
	None,
	PersonSegmentation,
	SceneDepth
};

/** Helper class to ensure the ARKit camera material is cooked. */
UCLASS()
class UARKitCameraOverlayMaterialLoader : public UObject
{
	GENERATED_BODY()
	
public:
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DefaultCameraOverlayMaterial;
	
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DepthOcclusionOverlayMaterial;
	
	UPROPERTY()
	TObjectPtr<UMaterialInterface> MatteOcclusionOverlayMaterial;
	
	UPROPERTY()
	TObjectPtr<UMaterialInterface> SceneDepthOcclusionMaterial;
	
	UPROPERTY()
	TObjectPtr<UMaterialInterface> SceneDepthColorationMaterial;
	
	UARKitCameraOverlayMaterialLoader()
	{
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultOverlayMaterialRef(*OverlayMaterialPath);
		DefaultCameraOverlayMaterial = DefaultOverlayMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> DepthOcclusionOverlayMaterialRef(*DepthOcclusionOverlayMaterialPath);
		DepthOcclusionOverlayMaterial = DepthOcclusionOverlayMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> MatteOcclusionOverlayMaterialRef(*MatteOcclusionOverlayMaterialPath);
		MatteOcclusionOverlayMaterial = MatteOcclusionOverlayMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> SceneDepthOcclusionMaterialRef(*SceneDepthOcclusionMaterialPath);
		SceneDepthOcclusionMaterial = SceneDepthOcclusionMaterialRef.Object;
		
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> SceneDepthColorationMaterialRef(*SceneDepthColorationMaterialPath);
		SceneDepthColorationMaterial = SceneDepthColorationMaterialRef.Object;
	}
	
	static const FString OverlayMaterialPath;
	static const FString DepthOcclusionOverlayMaterialPath;
	static const FString MatteOcclusionOverlayMaterialPath;
	static const FString SceneDepthOcclusionMaterialPath;
	static const FString SceneDepthColorationMaterialPath;
};

class FAppleARKitVideoOverlay : public FGCObject
{
public:
	FAppleARKitVideoOverlay();
	virtual ~FAppleARKitVideoOverlay();

	void SetCameraTexture(UARTextureCameraImage* InCameraImage);

	void RenderVideoOverlay_RenderThread(FRHICommandList& RHICmdList, const FSceneView& InView, struct FAppleARKitFrame& Frame, const float WorldToMeterScale);
	bool GetPassthroughCameraUVs_RenderThread(TArray<FVector2D>& OutUVs, const EDeviceScreenOrientation DeviceOrientation);

	void SetOverlayTexture(UARTextureCameraImage* InCameraImage);
	void SetOcclusionType(EARKitOcclusionType InOcclusionType);
	
	void UpdateSceneDepthTextures(UTexture* SceneDepthTexture, UTexture* DepthConfidenceTexture);
	
	UAppleARKitOcclusionTexture* GetOcclusionMatteTexture() const { return OcclusionMatteTexture; }
	UAppleARKitOcclusionTexture* GetOcclusionDepthTexture() const { return OcclusionDepthTexture; }

private:
	//~ FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAppleARKitVideoOverlay");
	}
	//~ FGCObject
	
	void RenderVideoOverlayWithMaterial(FRHICommandList& RHICmdList, const FSceneView& InView, struct FAppleARKitFrame& Frame, UMaterialInstanceDynamic* RenderingOverlayMaterial, const bool bRenderingOcclusion);
	void UpdateOcclusionTextures(const FAppleARKitFrame& Frame);
	void UpdateVideoTextures(const FAppleARKitFrame& Frame);
	
	void UpdateDebugOverlay();

	// 0 for landscape, 1 for portrait
	FBufferRHIRef OverlayVertexBufferRHI;
	FBufferRHIRef IndexBufferRHI;
	
	EARKitOcclusionType OcclusionType = EARKitOcclusionType::None;
	
#if SUPPORTS_ARKIT_3_0
	ARMatteGenerator* MatteGenerator = nullptr;
	id<MTLCommandQueue> CommandQueue = nullptr;
#endif
	
	bool bOcclusionDepthTextureRecentlyUpdated = false;
	
	TObjectPtr<UMaterialInstanceDynamic> MID_CameraOverlay = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> MID_DepthOcclusionOverlay = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> MID_MatteOcclusionOverlay = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> MID_SceneDepthOcclusion = nullptr;
	TObjectPtr<UMaterialInstanceDynamic> MID_SceneDepthColoration = nullptr;
	
	TObjectPtr<UAppleARKitOcclusionTexture> OcclusionMatteTexture = nullptr;
	TObjectPtr<UAppleARKitOcclusionTexture> OcclusionDepthTexture = nullptr;
	TObjectPtr<UARTextureCameraImage> CameraTexture = nullptr;
	TObjectPtr<UTexture> SceneDepthTexture = nullptr;
	TObjectPtr<UTexture> SceneDepthConfidenceTexture = nullptr;
	
	// 0: landscape, 1: portrait
	FVector2D UVOffsets[2] = { FVector2D::ZeroVector, FVector2D::ZeroVector };
};
