// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalTextureGuid.h"

/* GUID used to indicate that this external texture is the ARKit camera Y component. 0a417e66-bcda-4610-8594-92248c914a20 */
const FGuid ARKitPassthroughCameraExternalTextureYGuid(0x0A417E66, 0xBCDA4610, 0x85949224, 0x8C914A20);

/* GUID used to indicate that this external texture is the ARKit camera CbCr component. d61a2708-55fe-4cb1-9cde-df2a9bb2a2d2 */
const FGuid ARKitPassthroughCameraExternalTextureCbCrGuid(0xD61A2708, 0x55FE4CB1, 0x9CDEDF2A, 0x9BB2A2D2);

/* GUID used to indicate that this external texture is the ARKit camera depth texture. cba06c09-03b2-4818-8913-6424d99acf28 */
const FGuid ARKitPassthroughCameraExternalTextureDepthGuid(0xCBA06C09, 0x03B24818, 0x89136424, 0xD99ACF28);
