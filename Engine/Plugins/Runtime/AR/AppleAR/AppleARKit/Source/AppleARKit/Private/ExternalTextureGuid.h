// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

/* GUID used to indicate that this external texture is the ARKit camera Y component. 0a417e66-bcda-4610-8594-92248c914a20 */
extern const FGuid ARKitPassthroughCameraExternalTextureYGuid;

/* GUID used to indicate that this external texture is the ARKit camera CbCr component. d61a2708-55fe-4cb1-9cde-df2a9bb2a2d2 */
extern const FGuid ARKitPassthroughCameraExternalTextureCbCrGuid;

/* GUID used to indicate that this external texture is the ARKit camera depth texture. cba06c09-03b2-4818-8913-6424d99acf28 */
extern const FGuid ARKitPassthroughCameraExternalTextureDepthGuid;
