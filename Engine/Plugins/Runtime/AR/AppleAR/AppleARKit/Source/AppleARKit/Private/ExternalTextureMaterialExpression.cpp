// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalTextureMaterialExpression.h"
#include "MaterialCompiler.h"
#include "ExternalTextureGuid.h"

UDEPRECATED_MaterialExpressionARKitPassthroughCamera::UDEPRECATED_MaterialExpressionARKitPassthroughCamera(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITOR
int32 UDEPRECATED_MaterialExpressionARKitPassthroughCamera::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->TextureSample(
		Compiler->ExternalTexture((TextureType == TextureY) ? ARKitPassthroughCameraExternalTextureYGuid : ARKitPassthroughCameraExternalTextureCbCrGuid),
		Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false),
		EMaterialSamplerType::SAMPLERTYPE_Color);
}

int32 UDEPRECATED_MaterialExpressionARKitPassthroughCamera::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return INDEX_NONE;
}

void UDEPRECATED_MaterialExpressionARKitPassthroughCamera::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DEPRECATED, USE UARBlueprintLibrary::GetARTexture - ARKit Passthrough Camera"));
}
#endif
