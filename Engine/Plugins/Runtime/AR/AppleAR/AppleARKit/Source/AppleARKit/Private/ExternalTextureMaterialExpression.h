// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "ExternalTextureMaterialExpression.generated.h"

UENUM()
enum EARKitTextureType : int
{
	TextureY,
	TextureCbCr,
};

/**
* Implements a node sampling from the ARKit Passthrough external textures.
*/
UCLASS(collapsecategories, hidecategories = Object, Deprecated)
class UDEPRECATED_MaterialExpressionARKitPassthroughCamera : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	/** Only used if Coordinates is not hooked up */
	UPROPERTY(EditAnywhere, Category = UMaterialExpressionARKitPassthroughCamera)
	uint32 ConstCoordinate;

	UPROPERTY(EditAnywhere, Category = UMaterialExpressionARKitPassthroughCamera)
	TEnumAsByte<enum EARKitTextureType> TextureType;

#if WITH_EDITOR
	//~ UMaterialExpression
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ UMaterialExpression
#endif // WITH_EDITOR
};
