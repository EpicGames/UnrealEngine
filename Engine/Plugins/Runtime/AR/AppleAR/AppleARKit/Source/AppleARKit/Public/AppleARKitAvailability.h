// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#if PLATFORM_IOS && !PLATFORM_TVOS
	#include "Availability.h"
#endif

#if PLATFORM_IOS && !PLATFORM_TVOS

	// Check for ARKit 1.0
	#ifdef __IPHONE_11_0
		#define SUPPORTS_ARKIT_1_0 1
	#else
		#define SUPPORTS_ARKIT_1_0 0
	#endif

	// Check for ARKit 1.5
	#ifdef __IPHONE_11_3
		#define SUPPORTS_ARKIT_1_5 1
	#else
		#define SUPPORTS_ARKIT_1_5 0
	#endif

	// Check for ARKit 2.0
	#ifdef __IPHONE_12_0
		#define SUPPORTS_ARKIT_2_0 1
	#else
		#define SUPPORTS_ARKIT_2_0 0
	#endif

	// Check for ARKit 3.0
	#ifdef __IPHONE_13_0
		#define SUPPORTS_ARKIT_3_0 1
	#else
		#define SUPPORTS_ARKIT_3_0 0
	#endif
	
	// Check for ARKit 3.5
	#ifdef __IPHONE_13_4
		#define SUPPORTS_ARKIT_3_5 1
	#else
		#define SUPPORTS_ARKIT_3_5 0
	#endif

	// Check for ARKit 4.0
	#ifdef __IPHONE_14_0
		#define SUPPORTS_ARKIT_4_0 1
	#else
		#define SUPPORTS_ARKIT_4_0 0
	#endif

#else

	// No ARKit support
	#define SUPPORTS_ARKIT_1_0 0
	#define SUPPORTS_ARKIT_1_5 0
	#define SUPPORTS_ARKIT_2_0 0
	#define SUPPORTS_ARKIT_3_0 0
	#define SUPPORTS_ARKIT_3_5 0
	#define SUPPORTS_ARKIT_4_0 0
#endif

#define ADD_ARKIT_SUPPORT_CHECK(ARKitVersion,iOSVersion) \
static bool SupportsARKit##ARKitVersion() \
{ \
	static bool bSupportsFlag = false; \
	static bool bSupportChecked = false; \
	if (!bSupportChecked) \
	{ \
		bSupportChecked = true; \
		if (@available(iOS iOSVersion, *)) \
		{ \
			bSupportsFlag = true; \
		} \
	} \
	return bSupportsFlag; \
} \

class FAppleARKitAvailability
{
public:
#if SUPPORTS_ARKIT_1_0
	ADD_ARKIT_SUPPORT_CHECK(10, 11.0);
#else
	static bool SupportsARKit10() { return false; }
#endif
	
	
#if SUPPORTS_ARKIT_1_5
	ADD_ARKIT_SUPPORT_CHECK(15, 11.3);
#else
	static bool SupportsARKit15() { return false; }
#endif

	
#if SUPPORTS_ARKIT_2_0
	ADD_ARKIT_SUPPORT_CHECK(20, 12.0);
#else
	static bool SupportsARKit20() { return false; }
#endif

	
#if SUPPORTS_ARKIT_3_0
	ADD_ARKIT_SUPPORT_CHECK(30, 13.0);
#else
	static bool SupportsARKit30() { return false; }
#endif

	
#if SUPPORTS_ARKIT_3_5
	ADD_ARKIT_SUPPORT_CHECK(35, 13.4);
#else
	static bool SupportsARKit35() { return false; }
#endif
	
	
#if SUPPORTS_ARKIT_4_0
	ADD_ARKIT_SUPPORT_CHECK(40, 14.0);
#else
	static bool SupportsARKit40() { return false; }
#endif
};
