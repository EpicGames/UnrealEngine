// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitAvailability.h"
#include "Math/Transform.h"
#include "ARPin.h"
#include "Misc/Compression.h"
#include "Misc/Optional.h"
#include "ARTypes.h"
#include "ARTrackable.h"
#include "ARSessionConfig.h"
#include "Misc/Timecode.h"
#include "AppleARKitMeshData.h"
#include "ARGeoTrackingSupport.h"

#include "IAppleImageUtilsPlugin.h"

#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

#define AR_SAVE_WORLD_KEY 0x505A474A

enum class EARSaveWorldVersions : uint8
{
	Default = 1,
	AddAlignmentTransform = 2,
	Latest = AddAlignmentTransform,
};

struct FARWorldSaveHeader
{
	uint32 Magic;
	uint32 UncompressedSize;
	uint8 Version;
	
	FARWorldSaveHeader() :
		Magic(AR_SAVE_WORLD_KEY),
		UncompressedSize(0),
		Version((uint8)EARSaveWorldVersions::Latest)
	{
		
	}

	FARWorldSaveHeader(uint8* Header)
	{
		const FARWorldSaveHeader& Other = *(FARWorldSaveHeader*)Header;
		Magic = Other.Magic;
		UncompressedSize = Other.UncompressedSize;
		Version = Other.Version;
	}
};

#define AR_SAVE_WORLD_HEADER_SIZE (sizeof(FARWorldSaveHeader))

struct APPLEARKIT_API FAppleARKitConversion
{
	static inline float ToUEScale()
	{
		return 100.f;
	}

	static inline float ToARKitScale()
	{
		return 0.01f;
	}

#if SUPPORTS_ARKIT_1_0
	/**
	 * Convert's an ARKit 'Y up' 'right handed' coordinate system transform to Unreal's 'Z up' 
	 * 'left handed' coordinate system.
	 *
	 * Scale is removed because some ARKit transforms are less normalized than unreal requires.
	 */
	static inline FTransform ToFTransform(const matrix_float4x4& RawYUpMatrix, const FRotator& AdjustBy = FRotator::ZeroRotator)
	{
		// Conversion here is as per SteamVRHMD::ToFMatrix
		FMatrix RawYUpFMatrix(
			FPlane(RawYUpMatrix.columns[0][0], RawYUpMatrix.columns[0][1], RawYUpMatrix.columns[0][2], RawYUpMatrix.columns[0][3]),
			FPlane(RawYUpMatrix.columns[1][0], RawYUpMatrix.columns[1][1], RawYUpMatrix.columns[1][2], RawYUpMatrix.columns[1][3]),
			FPlane(RawYUpMatrix.columns[2][0], RawYUpMatrix.columns[2][1], RawYUpMatrix.columns[2][2], RawYUpMatrix.columns[2][3]),
			FPlane(RawYUpMatrix.columns[3][0], RawYUpMatrix.columns[3][1], RawYUpMatrix.columns[3][2], RawYUpMatrix.columns[3][3]));

		// ARKit, particularly when restoring a previously saved AR world map, sometimes has transforms who's normalization tolerance is not
		// tight enough to avoid ensures or even failure that result in identity rotations from FQuat.  So we remove scaling here.
		RawYUpFMatrix.RemoveScaling(UE_KINDA_SMALL_NUMBER / 100.0f);

		// Extract & convert translation
		FVector Translation = FVector( -RawYUpFMatrix.M[3][2], RawYUpFMatrix.M[3][0], RawYUpFMatrix.M[3][1] ) * ToUEScale();

		// Extract & convert rotation 
		FQuat RawRotation( RawYUpFMatrix );
		FQuat Rotation( -RawRotation.Z, RawRotation.X, RawRotation.Y, -RawRotation.W );
		if (!AdjustBy.IsNearlyZero())
		{
			Rotation = FQuat(AdjustBy) * Rotation;
		}

		return FTransform( Rotation, Translation );
	}

    /**
     * Convert's an Unreal 'Z up' transform to ARKit's 'Y up' 'right handed' coordinate system
     * 'left handed' coordinate system.
     *
     * Ignores scale.
     */
    static inline matrix_float4x4 ToARKitMatrix( const FTransform& InTransform, float WorldToMetersScale = 100.0f )
    {
		if(!ensure(WorldToMetersScale != 0.f))
		{
			WorldToMetersScale = 100.f;
		}

        matrix_float4x4 RetVal;

        const FVector   Translation = InTransform.GetLocation() / WorldToMetersScale;
        const FQuat     UnrealRotation = InTransform.GetRotation();
        const FQuat     ARKitRotation = FQuat(UnrealRotation.Y, UnrealRotation.Z, -UnrealRotation.X, UnrealRotation.W);

        const FMatrix   UnrealRotationMatrix = FRotationMatrix::Make(ARKitRotation);

        RetVal.columns[0][0] = UnrealRotationMatrix.M[0][0]; RetVal.columns[0][1] = UnrealRotationMatrix.M[0][1]; RetVal.columns[0][2] = -UnrealRotationMatrix.M[0][2]; RetVal.columns[0][3] = UnrealRotationMatrix.M[0][3];
        RetVal.columns[1][0] = UnrealRotationMatrix.M[1][0]; RetVal.columns[1][1] = UnrealRotationMatrix.M[1][1]; RetVal.columns[1][2] = UnrealRotationMatrix.M[1][2]; RetVal.columns[1][3] = UnrealRotationMatrix.M[1][3];
        RetVal.columns[2][0] = -UnrealRotationMatrix.M[2][0]; RetVal.columns[2][1] = UnrealRotationMatrix.M[2][1]; RetVal.columns[2][2] = UnrealRotationMatrix.M[2][2]; RetVal.columns[2][3] = UnrealRotationMatrix.M[2][3];
        RetVal.columns[3][0] = UnrealRotationMatrix.M[3][0]; RetVal.columns[3][1] = UnrealRotationMatrix.M[3][1]; RetVal.columns[3][2] = UnrealRotationMatrix.M[3][2]; RetVal.columns[3][3] = UnrealRotationMatrix.M[3][3];

        // Set the translation element
        RetVal.columns[3][2] = -Translation.X;
        RetVal.columns[3][0] = Translation.Y;
        RetVal.columns[3][1] = Translation.Z;

        return RetVal;
    }

	/**
	 * Convert's an ARKit 'Y up' 'right handed' coordinate system vector to Unreal's 'Z up' 
	 * 'left handed' coordinate system.
	 */
	static inline FVector ToFVector(const vector_float3& RawYUpVector)
	{
		return FVector( -RawYUpVector.z, RawYUpVector.x, RawYUpVector.y ) * ToUEScale();
	}

    /**
     * Convert's an Unreal's 'Z up' to ARKit's 'Y up' vector
     * 'left handed' coordinate system.
     */
    static inline vector_float3 ToARKitVector( const FVector& InFVector, float WorldToMetersScale = 100.0f )
    {
		if(!ensure(WorldToMetersScale != 0.f))
		{
			WorldToMetersScale = 100.f;
		}

        vector_float3 RetVal;
        RetVal.x = InFVector.Y;
        RetVal.y = InFVector.Z;
        RetVal.z = -InFVector.X;

        return RetVal / WorldToMetersScale;
    }

	static inline FGuid ToFGuid( uuid_t UUID )
	{
		const uint32* Components = reinterpret_cast<const uint32*>(UUID);
		FGuid AsGUID(Components[0], Components[1], Components[2], Components[3]);
		return AsGUID;
	}

	static inline FGuid ToFGuid( NSUUID* Identifier )
	{
		// Get bytes
		uuid_t UUID;
		[Identifier getUUIDBytes:UUID];

		// Set FGuid parts
		return ToFGuid( UUID );
	}

	static ARWorldAlignment ToARWorldAlignment( const EARWorldAlignment& InWorldAlignment );

	/**
	 * Coverts plane orientation
	 */
	static inline EARPlaneOrientation ToEARPlaneOrientation(ARPlaneAnchorAlignment Alignment)
	{
		EARPlaneOrientation RetVal = EARPlaneOrientation::Horizontal;
		if (Alignment == ARPlaneAnchorAlignmentVertical)
		{
			RetVal = EARPlaneOrientation::Vertical;
		}
		return RetVal;
	}

#if SUPPORTS_ARKIT_2_0
	static inline EARObjectClassification ToEARObjectClassification(ARPlaneClassification Classification)
	{
		switch(Classification)
		{
			case ARPlaneClassificationWall:
			{
				return EARObjectClassification::Wall;
			}

			case ARPlaneClassificationFloor:
			{
				return EARObjectClassification::Floor;
			}

			case ARPlaneClassificationCeiling:
			{
				return EARObjectClassification::Ceiling;
			}

			case ARPlaneClassificationTable:
			{
				return EARObjectClassification::Table;
			}

			case ARPlaneClassificationSeat:
			{
				return EARObjectClassification::Seat;
			}
		}
		return EARObjectClassification::Unknown;
	}
#endif
	
#if SUPPORTS_ARKIT_3_5
	static inline EARObjectClassification ToEARObjectClassification(ARMeshClassification Classification)
	{
		switch (Classification)
		{
			case ARMeshClassificationCeiling:
			{
				return EARObjectClassification::Ceiling;
			}

			case ARMeshClassificationDoor:
			{
				return EARObjectClassification::Door;
			}

			case ARMeshClassificationFloor:
			{
				return EARObjectClassification::Floor;
			}

			case ARMeshClassificationNone:
			{
				return EARObjectClassification::Unknown;
			}

			case ARMeshClassificationSeat:
			{
				return EARObjectClassification::Seat;
			}
				
			case ARMeshClassificationTable:
			{
				return EARObjectClassification::Table;
			}
				
			case ARMeshClassificationWall:
			{
				return EARObjectClassification::Wall;
			}
				
			case ARMeshClassificationWindow:
			{
				return EARObjectClassification::Window;
			}
		}
		
		return EARObjectClassification::Unknown;
	}
#endif

#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wpartial-availability"

#if SUPPORTS_ARKIT_1_5
	static ARVideoFormat* ToARVideoFormat(const FARVideoFormat& DesiredFormat, NSArray<ARVideoFormat*>* Formats, bool bUseOptimalFormat);
	
	static FARVideoFormat FromARVideoFormat(ARVideoFormat* Format);
	
	static TArray<FARVideoFormat> FromARVideoFormatArray(NSArray<ARVideoFormat*>* Formats);

	static NSSet* InitImageDetection(UARSessionConfig* SessionConfig, TMap<FString, UARCandidateImage*>& CandidateImages, TMap<FString, CGImageRef>& ConvertedCandidateImages);

	static NSArray<ARVideoFormat*>* GetSupportedVideoFormats(EARSessionType SessionType);
#endif

#if SUPPORTS_ARKIT_2_0
	static void InitImageDetection(UARSessionConfig* SessionConfig, ARImageTrackingConfiguration* ImageConfig, TMap<FString, UARCandidateImage*>& CandidateImages, TMap<FString, CGImageRef>& ConvertedCandidateImages);

	static AREnvironmentTexturing ToAREnvironmentTexturing(EAREnvironmentCaptureProbeType CaptureType);

	static ARWorldMap* ToARWorldMap(const TArray<uint8>& WorldMapData, TOptional<FTransform>& AlignmentTransform);

	static NSSet* ToARReferenceObjectSet(const TArray<UARCandidateObject*>& CandidateObjects, TMap< FString, UARCandidateObject* >& CandidateObjectMap);
#endif

	static ARConfiguration* ToARConfiguration(UARSessionConfig* SessionConfig, TMap<FString, UARCandidateImage*>& CandidateImages, TMap<FString, CGImageRef>& ConvertedCandidateImages, TMap<FString, UARCandidateObject*>& CandidateObjects, TOptional<FTransform>& InitialAlignmentTransform);
	
	static void ConfigureSessionTrackingFeatures(UARSessionConfig* SessionConfig, ARConfiguration* SessionConfiguration);

	#pragma clang diagnostic pop
#endif
	
#if SUPPORTS_ARKIT_1_0
	// Helper function to check if a particular session feature is supported with the specified session type
	static bool IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature);
	static bool IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod);
#else
	static bool IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) { return false; }
	static bool IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) { return false; }
#endif
	
#if SUPPORTS_ARKIT_4_0
	static EARGeoTrackingState ToGeoTrackingState(ARGeoTrackingState InState);
	static EARGeoTrackingStateReason ToGeoTrackingStateReason(ARGeoTrackingStateReason InReason);
	static EARGeoTrackingAccuracy ToGeoTrackingAccuracy(ARGeoTrackingAccuracy InAccuracy);
	static EARAltitudeSource ToAltitudeSource(ARAltitudeSource InSource);
#endif
		
#if SUPPORTS_ARKIT_3_0
	static ARFrameSemantics ToARFrameSemantics(EARSessionTrackingFeature SessionTrackingFeature);
	static FARPose2D ToARPose2D(const ARBody2D* InARPose2D);
	static FARPose3D ToARPose3D(const ARBodyAnchor* InARBodyAnchor);
    static FARPose3D ToARPose3D(const ARSkeleton3D* InSkeleton3D, bool bIdentityForUntracked);

private:
	static void ToSkeletonDefinition(const ARSkeletonDefinition* InARSkeleton, FARSkeletonDefinition& OutSkeletonDefinition);
#endif
};

#if SUPPORTS_ARKIT_1_0
enum class EAppleAnchorType : uint8
{
	Anchor,
	PlaneAnchor,
	FaceAnchor,
	ImageAnchor,
	EnvironmentProbeAnchor,
	ObjectAnchor,
	PoseAnchor,
	MeshAnchor,
	GeoAnchor,
	MAX
};

struct FAppleARKitAnchorData
{
	FAppleARKitAnchorData() = default;
	
	// Generic Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::Anchor )
		, AnchorGUID( InAnchorGuid )
		, bIsTracked(false)
	{
	}

	// Plane Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, FVector InCenter, FVector InExtent, EARPlaneOrientation InOrientation)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::PlaneAnchor )
		, AnchorGUID( InAnchorGuid )
		, Center(InCenter)
		, Extent(InExtent)
		, Orientation(InOrientation)
	{
	}

	// Face Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, FARBlendShapeMap InBlendShapes, TArray<FVector> InFaceVerts, TArray<FVector2D> InFaceUVData, FTransform InLeftEyeTransform, FTransform InRightEyeTransform, FVector InLookAtTarget, const FTimecode& InTimecode, uint32 InFrameRate)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::FaceAnchor )
		, AnchorGUID( InAnchorGuid )
		, BlendShapes( MoveTemp(InBlendShapes) )
		, FaceVerts( MoveTemp(InFaceVerts) )
		, FaceUVData( MoveTemp(InFaceUVData) )
		, LeftEyeTransform( InLeftEyeTransform )
		, RightEyeTransform( InRightEyeTransform )
		, LookAtTarget( InLookAtTarget )
		, Timecode( InTimecode )
		, FrameRate( InFrameRate )
	{
	}
	
	// Image Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, EAppleAnchorType InAnchorType, FString InDetectedAnchorName)
		: Transform( InTransform )
		, AnchorType( InAnchorType )
		, AnchorGUID( InAnchorGuid )
		, DetectedAnchorName( MoveTemp(InDetectedAnchorName) )
    {
    }
	
	// Probe Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, FVector InExtent, id<MTLTexture> InProbeTexture)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::EnvironmentProbeAnchor )
		, AnchorGUID( InAnchorGuid )
		, Extent(InExtent)
		, ProbeTexture(InProbeTexture)
	{
	}

	// Body Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, FARPose3D InTrackedPose)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::PoseAnchor )
		, AnchorGUID( InAnchorGuid )
		, TrackedPose( MoveTemp(InTrackedPose) )
	{
	}
	
	// Mesh Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, FARKitMeshData::MeshDataPtr InMeshData)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::MeshAnchor )
		, AnchorGUID( InAnchorGuid )
		, MeshData( InMeshData )
	{
	}
	
	// Geo Anchor
	FAppleARKitAnchorData(FGuid InAnchorGuid, FTransform InTransform, float InLongitude, float InLatitude, float InAltitudeMeters, EARAltitudeSource InAltitudeSource)
		: Transform( InTransform )
		, AnchorType( EAppleAnchorType::GeoAnchor )
		, AnchorGUID( InAnchorGuid )
		, Longitude( InLongitude )
		, Latitude( InLatitude )
		, AltitudeMeters( InAltitudeMeters )
		, AltitudeSource( InAltitudeSource )
	{
	}

	FTransform Transform = FTransform::Identity;
	EAppleAnchorType AnchorType = EAppleAnchorType::Anchor;
	FGuid AnchorGUID;
	FVector Center = FVector::ZeroVector;
	FVector Extent = FVector::ZeroVector;
	EARPlaneOrientation Orientation = EARPlaneOrientation::Horizontal;
	EARObjectClassification ObjectClassification = EARObjectClassification::NotApplicable;
	/** Set by the session config to detemine whether to generate geometry or not */
	static bool bGenerateGeometry;
	TArray<FVector> BoundaryVerts;
	TArray<FVector> Vertices;
	TArray<uint32> Indices;

	FARBlendShapeMap BlendShapes;
	TArray<FVector> FaceVerts;
	TArray<FVector2D> FaceUVData;
	// Note: the index buffer never changes so can be safely read once
	static TArray<int32> FaceIndices;

	FString DetectedAnchorName;

	id<MTLTexture> ProbeTexture = nullptr;

	FTransform LeftEyeTransform = FTransform::Identity;
	FTransform RightEyeTransform = FTransform::Identity;
	FVector LookAtTarget = FVector::ZeroVector;
	double Timestamp = 0.0;
	uint32 FrameNumber = 0;
	FTimecode Timecode;
	uint32 FrameRate = 0;

	/** Only valid for tracked real world objects (face, images) */
	bool bIsTracked = true;
	
	/** Only valid if this is a body anchor */
	FARPose3D TrackedPose;
	
	FARKitMeshData::MeshDataPtr MeshData;
	
	FString AnchorName;
	
	// Geo Anchor Properties
	float Longitude = 0.f;
	float Latitude = 0.f;
	float AltitudeMeters = 0.f;
	EARAltitudeSource AltitudeSource = EARAltitudeSource::Unknown;
	
	// only need to be initialized once
	static TSharedPtr<FARPose3D> BodyRefPose;
};
#endif

namespace ARKitUtil
{
	static UARPin* PinFromComponent( const USceneComponent* Component, const TArray<UARPin*>& InPins )
	{
		for (UARPin* Pin : InPins)
		{
			if (Pin->GetPinnedComponent() == Component)
			{
				return Pin;
			}
		}

		return nullptr;
	}

	static TArray<UARPin*> PinsFromGeometry( const UARTrackedGeometry* Geometry, const TArray<UARPin*>& InPins )
	{
		TArray<UARPin*> OutPins;
		for (UARPin* Pin : InPins)
		{
			if (Pin->GetTrackedGeometry() == Geometry)
			{
				OutPins.Add(Pin);
			}
		}

		return OutPins;
	}
}
