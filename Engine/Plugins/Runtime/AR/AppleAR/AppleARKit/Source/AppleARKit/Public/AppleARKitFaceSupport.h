// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

#include "AppleARKitAvailability.h"

#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

#include "ARSessionConfig.h"

struct FAppleARKitAnchorData;
struct FARVideoFormat;
struct FGuid;
class UTimecodeProvider;
enum class EARFaceTrackingUpdate : uint8;

class IAppleARKitFaceSupport :
	public IModularFeature
{
public:
#if SUPPORTS_ARKIT_1_0
	/**
	 * Converts a set of generic ARAnchors into their face anchor equivalents without exposing the main code to the face APIs
	 *
	 * @param NewAnchors the list of anchors to convert to our intermediate format
	 * @param AdjustBy the additional rotation to apply to put the rotation in the proper space (camera alignment only)
	 * @param UpdateSetting whether to just update curves or geo too
	 *
	 * @return the set of face anchors to dispatch
	 */
	virtual TArray<TSharedPtr<FAppleARKitAnchorData>> MakeAnchorData(NSArray<ARAnchor*>* NewAnchors, const FRotator& AdjustBy, EARFaceTrackingUpdate UpdateSetting) { return TArray<TSharedPtr<FAppleARKitAnchorData>>(); }

	/**
	 * Publishes any face AR data that needs to be sent to LiveLink. Done as a separate step because MakeAnchorData is called
	 * on an arbitrary thread and we can't access UObjects there safely
	 *
	 * @param SessionGuid the Guid of the session
	 * @param AnchorList the list of anchors to publish to LiveLink
	 *
	 * @return the set of face anchors to dispatch
	 */
	virtual void PublishLiveLinkData(const FGuid& SessionGuid, TSharedPtr<FAppleARKitAnchorData> Anchor) { }

	/**
	 * Creates a face ar specific configuration object if that is requested without exposing the main code to the face APIs
	 *
	 * @param SessionConfig the UE configuration object that needs processing
	 * @param InProvider the custom timecode provider to use
	 */
	virtual ARConfiguration* ToARConfiguration(UARSessionConfig* SessionConfig, UTimecodeProvider* InProvider) { return nullptr; }

	/**
	 * @return whether this device supports face ar
	 */
	virtual bool DoesSupportFaceAR() { return false; }
#endif
#if SUPPORTS_ARKIT_1_5
	/**
	 * @return the supported video formats by the face ar device
	 */
	virtual NSArray<ARVideoFormat*>* GetSupportedVideoFormats() const { return nullptr; }
#endif
	
#if SUPPORTS_ARKIT_3_0
	/**
	 * @return If the desired frame semantics is supported by AR face tracking
	 */
	virtual bool IsARFrameSemanticsSupported(ARFrameSemantics InSemantics) const { return false; }
#endif
	
	/** @return the max number of faces can be tracked at the same time */
	virtual int32 GetNumberOfTrackedFacesSupported() const = 0;
	
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("AppleARKitFaceSupport"));
		return FeatureName;
	}
};
