// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitAvailability.h"
#include "MRMeshComponent.h"
#include "ARComponent.h"
#include "HAL/CriticalSection.h"

#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif


/** Represents a cached version of the data coming from an ARKit mesh anchor */
class FARKitMeshData : public IMRMesh::FBrickDataReceipt
{
public:
	~FARKitMeshData();
	
	typedef TSharedPtr<FARKitMeshData, ESPMode::ThreadSafe> MeshDataPtr;

	/** Clears the cached data for all the mesh anchors */
	static void ClearAllMeshData();
	
	/** Removes the cached data for a particular mesh anchor, identified by the GUID */
	static void RemoveMeshData(const FGuid& InGuid);
	
	/** Updates a MR mesh with the data cached in a particular mesh data object */
	static void UpdateMRMesh(const FTransform& MeshTransform, MeshDataPtr MeshData, UMRMeshComponent* InMeshComponent);
	
	/** Returns the cached data for a particular mesh anchor, identified by the GUID */
	static MeshDataPtr GetMeshData(const FGuid& InGuid);
	
	/**
	 * Gets the object classification for a location in the world space
	 * See UARBlueprintLibrary::GetObjectClassificationAtLocation
	 */
	bool GetClassificationAtLocation(const FVector& InWorldLocation, const FTransform& InLocalToWorldTransform, uint8& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff);
	
#if SUPPORTS_ARKIT_3_5
	/** Grabs the data from the incoming mesh anchor and cache it in a data object */
	static TSharedPtr<FARKitMeshData, ESPMode::ThreadSafe> CacheMeshData(const FGuid& InGuid, ARMeshAnchor* InMeshAnchor);
	
private:
	void UpdateMeshData(ARMeshAnchor* InMeshAnchor);
	void UpdateMeshData();
	void ClearData();
	
	/** The ARKit mesh geometry waiting to be processed */
	ARMeshGeometry* PendingMeshGeometry = nullptr;
	
	/** Lock protecting the access to PendingMeshGeometry */
	FCriticalSection PendingMeshLock;
#endif
	
private:
	/** Mesh vertices in the anchor's local space */
	TArray<FVector3f> Vertices;
	
	/** Mesh tangents in the anchor's local space */
	TArray<FPackedNormal> TangentData;
	
	/** Mesh face indices */
	TArray<MRMESH_INDEX_TYPE> Indices;
	
	/** Classification ID for each face, might be empty if mesh classification is not used */
	TArray<uint8> Classifications;
	
	/** Dummy UV data, filled with zeros */
	TArray<FVector2D> DummyUVData;
	
	/** Dummy color data, filled with zeros */
	TArray<FColor> DummyColorData;
	
	/** Bounding box of the vertices, in the anchor's local space */
	FBox BoundingBox;
	
	/** Temporary data used for calculating the normals from vertices */
	TArray<FAccumulatedNormal> AccumulatedNormals;
	
	/** Global cached mesh data records, one for each anchor */
	static TMap<FGuid, MeshDataPtr> CachedMeshData;
	
	/** Lock protecting the access to CachedMeshData */
	static FCriticalSection GlobalLock;
};
