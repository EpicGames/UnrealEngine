// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

#include "AppleARKitAvailability.h"

#if SUPPORTS_ARKIT_3_0
	#import <ARKit/ARKit.h>
#endif

struct FAppleARKitAnchorData;

class IAppleARKitPoseTrackingLiveLink
	: public IModularFeature
{
public:
	/**
	 * Publishes any body tracking AR data that needs to be sent to LiveLink. Done as a separate step because MakeAnchorData is called
	 * on an arbitrary thread and we can't access UObjects there safely
	 *
	 * @param AnchorList the list of anchors to publish to LiveLink
	 *
	 * @return the set of body tracking anchors to dispatch
	 */
	virtual void PublishLiveLinkData(TSharedPtr<FAppleARKitAnchorData> Anchor) { }

	virtual const TArray<FTransform>* GetRefPoseTransforms() const { return nullptr; }
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("AppleARKitPoseTrackingLiveLink"));
		return FeatureName;
	}
};
