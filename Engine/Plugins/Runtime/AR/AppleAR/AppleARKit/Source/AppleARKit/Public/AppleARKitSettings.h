// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ARTrackable.h"
#include "Engine/TimecodeProvider.h"

#include "AppleImageUtilsTypes.h"
// MERGE-TODO - this now seems to be private
#include "../Private/AppleARKitTimecodeProvider.h"

#include "AppleARKitSettings.generated.h"

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARFaceTrackingFileWriterType : uint8
{
	/** Disables creation of a file writer */
	None,
	/** Comma delimited file, one row per captured frame */
	CSV,
	/** JSON object array, one frame object per captured frame */
	JSON
};

UENUM(BlueprintType, Category="AR AugmentedReality")
enum class ELivelinkTrackingType : uint8
{
	FaceTracking,
	PoseTracking
};


UCLASS(Config=Engine, defaultconfig)
class APPLEARKIT_API UAppleARKitSettings :
	public UObject,
	public FSelfRegisteringExec
{
	GENERATED_BODY()

public:
	UAppleARKitSettings()
		: bRequireARKitSupport(true)
		, bFaceTrackingLogData(false)
		, bFaceTrackingWriteEachFrame(false)
		, FaceTrackingFileWriterType(EARFaceTrackingFileWriterType::None)
		, bShouldWriteCameraImagePerFrame(false)
		, WrittenCameraImageScale(1.f)
		, WrittenCameraImageQuality(85)
		, LiveLinkPublishingPort(11111)
		, DefaultFaceTrackingLiveLinkSubjectName(FName("iPhoneXFaceAR"))
		, DefaultPoseTrackingLiveLinkSubjectName(FName("PoseTracking"))
		, DefaultFaceTrackingDirection(EARFaceTrackingDirection::FaceRelative)
		, bAdjustThreadPrioritiesDuringARSession(false)
		, GameThreadPriorityOverride(47)
		, RenderThreadPriorityOverride(45)
		, ARKitTimecodeProvider(TEXT("/Script/AppleARKit.AppleARKitTimecodeProvider"))
	{
	}

public:
	static UTimecodeProvider* GetTimecodeProvider();
	static void CreateFaceTrackingLogDir();
	static void CreateImageLogDir();

// Accessors to the properties
	FString GetFaceTrackingLogDir();
	bool IsLiveLinkEnabledForFaceTracking();
	bool IsLiveLinkEnabledForPoseTracking();
	bool IsFaceTrackingLoggingEnabled();
	bool ShouldFaceTrackingLogPerFrame();
	EARFaceTrackingFileWriterType GetFaceTrackingFileWriterType();
	bool ShouldWriteCameraImagePerFrame();
	float GetWrittenCameraImageScale();
	int32 GetWrittenCameraImageQuality();
	ETextureRotationDirection GetWrittenCameraImageRotation();
	int32 GetLiveLinkPublishingPort();
	FName GetFaceTrackingLiveLinkSubjectName();
	FName GetPoseTrackingLiveLinkSubjectName();
	EARFaceTrackingDirection GetFaceTrackingDirection();
	bool ShouldAdjustThreadPriorities();
	int32 GetGameThreadPriorityOverride();
	int32 GetRenderThreadPriorityOverride();

protected:
	//~ FSelfRegisteringExec
	virtual bool Exec_Runtime(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ FSelfRegisteringExec

	/** Where to write the curve files and image files */
    UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings", meta=(ToolTip="When True the project can only be installed on devices that support ARKit."))
    bool bRequireARKitSupport;
    
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	FString FaceTrackingLogDir;

	/** Livelink tracking type. To publish face blend shapes, or body pose data to LiveLink, or none */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	TArray<ELivelinkTrackingType> LivelinkTrackingTypes;

	/** Whether file writing is enabled at all or not */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	bool bFaceTrackingLogData;

	/** Whether to publish each frame or when the "FaceAR WriteCurveFile */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	bool bFaceTrackingWriteEachFrame;

	/** The type of face AR publisher that writes to disk to create */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	EARFaceTrackingFileWriterType FaceTrackingFileWriterType;

	/** Whether to publish the camera image each frame */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	bool bShouldWriteCameraImagePerFrame;

	/** The scale to write the images at. Used to reduce data footprint */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	float WrittenCameraImageScale;

	/** The quality setting to generate the jpeg images at. Defaults to 85, which is "high quality". Lower values reduce data footprint */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	int32 WrittenCameraImageQuality;

	/** Defaults to none. Use Right when in portrait mode */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	ETextureRotationDirection WrittenCameraImageRotation;

	/** The port to use when listening/sending LiveLink face blend shapes via the network */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	int32 LiveLinkPublishingPort;
	
	/**
	 * The default name to use when publishing face tracking name
	 * If multiple faces are tracked, the subject name for the faces will be:
	 * #1: DefaultFaceTrackingLiveLinkSubjectName
	 * #2: DefaultFaceTrackingLiveLinkSubjectName-1
	 * #3: DefaultFaceTrackingLiveLinkSubjectName-2, etc
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	FName DefaultFaceTrackingLiveLinkSubjectName;
	
	/** The default name to use when publishing pose tracking name */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	FName DefaultPoseTrackingLiveLinkSubjectName;

	/** The default tracking to use when tracking face blend shapes (face relative or mirrored). Defaults to face relative */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	EARFaceTrackingDirection DefaultFaceTrackingDirection;

	/** Whether to adjust thread priorities during an AR session or not */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	bool bAdjustThreadPrioritiesDuringARSession;

	/** The game thread priority to change to when an AR session is running, default is 47 */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	int32 GameThreadPriorityOverride;
	
	/** The render thread priority to change to when an AR session is running, default is 45 */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	int32 RenderThreadPriorityOverride;

	/**
	 * Used to specify the timecode provider to use when identifying when an update occurred.
	 * Useful when using external timecode generators to sync multiple devices/machines
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="AR Settings")
	FString ARKitTimecodeProvider;

private:
	/** Used because these defaults need to be read on multiple threads */
	FCriticalSection CriticalSection;
};
