// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitAvailability.h"
#include "ARTextures.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if PLATFORM_APPLE
	#import <CoreVideo/CoreVideo.h>
	#import <AVFoundation/AVFoundation.h>
#endif
#include "AppleImageUtilsTypes.h"
#include "AppleARKitTextures.generated.h"


struct FAppleARKitFrame;

/** Parameters used to scale and blur an image */
struct APPLEARKIT_API FImageBlurParams
{
	/** Sigma for the gaussian blur operation */
	float GaussianBlurSigma = 0.f;
	
	/** Scale applied to the original image before it's blurred */
	float ImageSizeScale = 1.f;
};

UCLASS(NotBlueprintType)
class APPLEARKIT_API UAppleARKitTextureCameraImage :
	public UARTextureCameraImage,
	public IAppleImageInterface
{
	GENERATED_UCLASS_BODY()

public:
	// UTexture interface implementation
	virtual void BeginDestroy() override;
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	// End UTexture interface
	
	virtual EAppleTextureType GetTextureType() const override { return EAppleTextureType::PixelBuffer; }
	
#if PLATFORM_APPLE
	void UpdateCameraImage(float InTimestamp, CVPixelBufferRef InCameraImage, EPixelFormat InPixelFormat, const CFStringRef ColorSpace = kCGColorSpaceGenericRGBLinear, FImageBlurParams BlurParams = {});
	
	/** Returns the cached camera image. You must retain this if you hold onto it */
	CVPixelBufferRef GetCameraImage() const { return CameraImage; }

	// IAppleImageInterface interface implementation
	virtual CVPixelBufferRef GetPixelBuffer() const override { return CameraImage; }
	// End IAppleImageInterface interface
	
private:
	/** The Apple specific representation of the ar camera image */
	CVPixelBufferRef CameraImage = nullptr;
#endif
};

UCLASS(NotBlueprintType)
class APPLEARKIT_API UAppleARKitTextureCameraDepth :
	public UARTextureCameraDepth
{
	GENERATED_UCLASS_BODY()

public:
	// UTexture interface implementation
	virtual void BeginDestroy() override;
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_TextureExternal; }
	// End UTexture interface

#if SUPPORTS_ARKIT_1_0
	/** Sets any initialization data */
	virtual void Init(float InTimestamp, AVDepthData* InCameraDepth);
#endif

#if PLATFORM_APPLE
	/** Returns the cached camera depth. You must retain this if you hold onto it */
	AVDepthData* GetCameraDepth() { return CameraDepth; }

private:
	/** The Apple specific representation of the ar depth image */
	AVDepthData* CameraDepth = nullptr;
#endif
};

UCLASS(NotBlueprintType)
class APPLEARKIT_API UAppleARKitEnvironmentCaptureProbeTexture :
	public UAREnvironmentCaptureProbeTexture,
	public IAppleImageInterface
{
	GENERATED_UCLASS_BODY()
	
public:
	
	// UTexture interface implementation
	virtual void BeginDestroy() override;
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_TextureExternal; }
	// End UTexture interface
	
	virtual EAppleTextureType GetTextureType() const override { return EAppleTextureType::MetalTexture; }

#if PLATFORM_APPLE
	/** Sets any initialization data */
	virtual void Init(float InTimestamp, id<MTLTexture> InEnvironmentTexture);

	// IAppleImageInterface interface implementation
	virtual id<MTLTexture> GetMetalTexture() const override { return MetalTexture; }
	// End IAppleImageInterface interface
	
private:
	/** The Apple specific representation of the ar environment texture */
	id<MTLTexture> MetalTexture = nullptr;
#endif
};

UCLASS(NotBlueprintType)
class APPLEARKIT_API UAppleARKitOcclusionTexture :
	public UARTexture,
	public IAppleImageInterface
{
	GENERATED_UCLASS_BODY()
	
public:
	// UTexture interface implementation
	virtual void BeginDestroy() override;
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	// End UTexture interface
	
	virtual EAppleTextureType GetTextureType() const override { return EAppleTextureType::MetalTexture; }
	
#if PLATFORM_APPLE
	void SetMetalTexture(float InTimestamp, id<MTLTexture> InMetalTexture, EPixelFormat PixelFormat, const CFStringRef ColorSpace = kCGColorSpaceGenericRGBLinear);
	
	// IAppleImageInterface interface implementation
	virtual id<MTLTexture> GetMetalTexture() const override;
	// End IAppleImageInterface interface
	
private:
	/** The Apple specific representation of the ar camera image */
	id<MTLTexture> MetalTexture = nullptr;
	mutable FCriticalSection MetalTextureLock;
#endif
};

UCLASS(NotBlueprintType)
class APPLEARKIT_API UAppleARKitCameraVideoTexture : public UARTextureCameraImage, public IAppleImageInterface
{
	GENERATED_UCLASS_BODY()

public:
	// UTexture interface implementation
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	// End UTexture interface

	void Init();
	void UpdateFrame(const FAppleARKitFrame& InFrame);
	
	virtual EAppleTextureType GetTextureType() const override { return EAppleTextureType::MetalTexture; }

#if SUPPORTS_ARKIT_1_0
	virtual id<MTLTexture> GetMetalTexture() const override;
#endif
};
