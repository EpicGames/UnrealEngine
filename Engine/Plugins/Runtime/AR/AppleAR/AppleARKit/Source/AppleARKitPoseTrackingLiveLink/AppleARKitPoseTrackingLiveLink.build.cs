// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AppleARKitPoseTrackingLiveLink : ModuleRules
{
	public AppleARKitPoseTrackingLiveLink(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
        //OptimizeCode = CodeOptimization.Never;	
		
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Engine",
		});
			
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"HeadMountedDisplay",
			"XRBase",
			"LiveLinkAnimationCore",
			"LiveLinkInterface",
			"AppleARKit",
			"AppleImageUtils",
			"ARUtilities",
		});
		
		
		DynamicallyLoadedModuleNames.AddRange(new string[]
		{
		});

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PublicFrameworks.Add( "ARKit" );
		}
	}
}
