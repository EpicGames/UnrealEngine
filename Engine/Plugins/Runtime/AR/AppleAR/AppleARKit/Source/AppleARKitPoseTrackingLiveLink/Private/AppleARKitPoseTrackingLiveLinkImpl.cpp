// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitPoseTrackingLiveLinkImpl.h"
#include "AppleARKitSettings.h"
#include "AppleARKitConversion.h"
#include "ARSystem.h"
#include "AppleARKitPoseTrackingLiveLinkModule.h"

FAppleARKitPoseTrackingLiveLink::FAppleARKitPoseTrackingLiveLink() :
	LocalDeviceId(FName(*FPlatformMisc::GetDeviceId()))	// Generate our device id
{
}

FAppleARKitPoseTrackingLiveLink::~FAppleARKitPoseTrackingLiveLink()
{
	// Should only be called durirng shutdown
	check(IsEngineExitRequested());
}

void FAppleARKitPoseTrackingLiveLink::Init()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

void FAppleARKitPoseTrackingLiveLink::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

#if SUPPORTS_ARKIT_3_0
void FAppleARKitPoseTrackingLiveLink::PublishLiveLinkData(TSharedPtr<FAppleARKitAnchorData> Anchor)
{
	static bool bNeedsInit = true;
	if (bNeedsInit)
	{
		bNeedsInit = false;
	    // Create our LiveLink provider if the project setting is enabled
		if (GetMutableDefault<UAppleARKitSettings>()->IsLiveLinkEnabledForPoseTracking())
		{
			LiveLinkSource = FAppleARKitPoseTrackingLiveLinkSourceFactory::CreateLiveLinkSource();
		}
	}

	if (LiveLinkSource.IsValid())
	{
		PoseTrackingLiveLinkSubjectName = GetMutableDefault<UAppleARKitSettings>()->GetPoseTrackingLiveLinkSubjectName();
        LiveLinkSource->UpdateLiveLink(PoseTrackingLiveLinkSubjectName, Anchor->Timecode, Anchor->FrameRate, Anchor->TrackedPose, *FAppleARKitAnchorData::BodyRefPose, LocalDeviceId);
	}
}
#endif

const TArray<FTransform>* FAppleARKitPoseTrackingLiveLink::GetRefPoseTransforms() const
{
	if (LiveLinkSource.IsValid())
	{
		return LiveLinkSource->GetRefPoseTransforms();
	}

	return nullptr;
}
