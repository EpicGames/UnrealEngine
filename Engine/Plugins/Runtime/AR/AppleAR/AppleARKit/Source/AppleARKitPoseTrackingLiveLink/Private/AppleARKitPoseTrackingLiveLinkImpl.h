// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AppleARKitPoseTrackingLiveLink.h"
#include "AppleARKitPoseTrackingLiveLinkSourceFactory.h"
#include "AppleARKitConversion.h"


class APPLEARKITPOSETRACKINGLIVELINK_API FAppleARKitPoseTrackingLiveLink :
	public IAppleARKitPoseTrackingLiveLink,
	public TSharedFromThis<FAppleARKitPoseTrackingLiveLink, ESPMode::ThreadSafe>
{
public:
	FAppleARKitPoseTrackingLiveLink();
	virtual ~FAppleARKitPoseTrackingLiveLink();

	void Init();
	void Shutdown();

private:
#if SUPPORTS_ARKIT_3_0
	// ~IAppleARKitPoseTrackingLiveLink
	void PublishLiveLinkData(TSharedPtr<FAppleARKitAnchorData> Anchor) override;
	// ~IAppleARKitPoseTrackingLiveLink

#endif

	// ~IAppleARKitPoseTrackingLiveLink
	const TArray<FTransform>* GetRefPoseTransforms() const override;
	// ~IAppleARKitPoseTrackingLiveLink	

	/** If requested, publishes face ar updates to LiveLink for the animation system to use */
	TSharedPtr<ILiveLinkSourceARKitPoseTracking> LiveLinkSource;
	/** Copied from the UARSessionConfig project settings object */
	FName PoseTrackingLiveLinkSubjectName;
	/** The id of this device */
	FName LocalDeviceId;
};

