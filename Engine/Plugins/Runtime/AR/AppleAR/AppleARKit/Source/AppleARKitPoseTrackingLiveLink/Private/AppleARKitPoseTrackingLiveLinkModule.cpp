// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitPoseTrackingLiveLinkModule.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "CoreGlobals.h"
#include "AppleARKitPoseTrackingLiveLinkImpl.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "BonePose.h"
#include "BoneIndices.h"


IMPLEMENT_MODULE(FAppleARKitPoseTrackingLiveLinkModule, AppleARKitPoseTrackingLiveLink);

DEFINE_LOG_CATEGORY(LogAppleARKitPoseTracking);

TSharedPtr<FAppleARKitPoseTrackingLiveLink, ESPMode::ThreadSafe> PoseTrackingLiveLinkInstance;

void FAppleARKitPoseTrackingLiveLinkModule::StartupModule()
{
	ensureMsgf(FModuleManager::Get().LoadModule("AppleARKit"), TEXT("ARKitPoseTrackingLiveLink depends on the AppleARKit module."));

	PoseTrackingLiveLinkInstance = MakeShared<FAppleARKitPoseTrackingLiveLink, ESPMode::ThreadSafe>();
	PoseTrackingLiveLinkInstance->Init();

	FQuat AppleForward = FVector(-1, 0, 0).ToOrientationQuat();
	FQuat MeshForward = FVector(0, 1, 0).ToOrientationQuat();;
	const auto AppleSpaceToMeshSpace = FTransform(AppleForward).Inverse() * FTransform(MeshForward);
	
	IModularFeatures::Get().RegisterModularFeature(IARLiveLinkRetargetingLogic::GetModularFeatureName(), this);
}

void FAppleARKitPoseTrackingLiveLinkModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IARLiveLinkRetargetingLogic::GetModularFeatureName(), this);
	
	PoseTrackingLiveLinkInstance->Shutdown();
	PoseTrackingLiveLinkInstance = nullptr;
}

FTransform GetRefMeshBoneToComponent(const FCompactPose& OutPose, int BoneIndex)
{
	FTransform Transform = FTransform::Identity;
	while (BoneIndex >= 0)
	{
		FCompactPoseBoneIndex CPIndex = OutPose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(BoneIndex));
		if (CPIndex != INDEX_NONE)
		{
			Transform *= OutPose.GetRefPose(CPIndex);
			BoneIndex = OutPose.GetBoneContainer().GetParentBoneIndex(BoneIndex);
		}
		else
		{
			break;
		}
	}

	return Transform;
}

static FTransform GetAppleBoneToComponentTransform(const TArray<int>& ParentIndices, const TArray<FTransform>& Transforms, int BoneIndex)
{
	FTransform BoneToComponent = FTransform::Identity;
	
	int Index = BoneIndex;
	while (Index >= 0)
	{
		BoneToComponent *= Transforms[Index];
		Index = ParentIndices[Index];
	}
	
	return BoneToComponent;
}

FTransform GetAppleBoneToComponentTransformFromFrameData(const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, int BoneIndex)
{
	return GetAppleBoneToComponentTransform(InSkeletonData->GetBoneParents(), InFrameData->Transforms, BoneIndex);
}

FTransform GetAppleBoneToComponentTransformFromRefSkeleton(const FLiveLinkSkeletonStaticData* InSkeletonData, int BoneIndex)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(IAppleARKitPoseTrackingLiveLink::GetModularFeatureName()))
	{
		IAppleARKitPoseTrackingLiveLink* BodyTrackLiveLink = &IModularFeatures::Get().GetModularFeature<IAppleARKitPoseTrackingLiveLink>(IAppleARKitPoseTrackingLiveLink::GetModularFeatureName());
		if (auto RefTransforms = BodyTrackLiveLink->GetRefPoseTransforms())
		{
			return GetAppleBoneToComponentTransform(InSkeletonData->GetBoneParents(), *RefTransforms, BoneIndex);
		}
	}

	return FTransform::Identity;
}

void FAppleARKitPoseTrackingLiveLinkModule::BuildPoseFromAnimationData(const UARLiveLinkRetargetAsset& SourceAsset, float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData,
																	   const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose)
{
	check(InSkeletonData);
	check(InFrameData);

	// Transform Bone Names
	const TArray<FName>& AppleARKitBoneNames = InSkeletonData->GetBoneNames();

	TArray<FName, TMemStackAllocator<>> MeshBoneNames;
	MeshBoneNames.Reserve(AppleARKitBoneNames.Num());

	TArray<FTransform> MeshBoneToComponents;
	const auto MeshBoneNum = OutPose.GetBoneContainer().GetReferenceSkeleton().GetNum();
	MeshBoneToComponents.Reserve(MeshBoneNum);
	for (auto i = 0; i < MeshBoneNum; ++i)
	{
		MeshBoneToComponents.Emplace(GetRefMeshBoneToComponent(OutPose, i));
	}

	for (const FName& AppleARKitBoneName : AppleARKitBoneNames)
	{
		const FName MeshBoneName = SourceAsset.GetRemappedBoneName(AppleARKitBoneName);
		MeshBoneNames.Add(MeshBoneName);
	}

	const FQuat ApplyARKitForwardDir = SourceAsset.ARKitPoseTrackingConfig.HumanForward.ToOrientationQuat();
	const FQuat MeshForwardDir = SourceAsset.ARKitPoseTrackingConfig.MeshForward.ToOrientationQuat();
	const auto AppleSpaceToMeshSpace = FTransform(ApplyARKitForwardDir).Inverse() * FTransform(MeshForwardDir);

	for (auto AppleBoneIndex = 0; AppleBoneIndex < MeshBoneNames.Num(); ++AppleBoneIndex)
	{
		// for each bone, find out its "offset" from ref pose to the actual pose in apple arkit component space
		// and then apply it to the unreal skeleton
		const auto& MeshBoneName = MeshBoneNames[AppleBoneIndex];
		const auto MeshBoneIndex = OutPose.GetBoneContainer().GetPoseBoneIndexForBoneName(MeshBoneName);
		if (MeshBoneIndex != INDEX_NONE)
		{
            const auto AppleARKitBoneToComponent_Ref = GetAppleBoneToComponentTransformFromRefSkeleton(InSkeletonData, AppleBoneIndex);
			const auto AppleARKitBoneToComponent = GetAppleBoneToComponentTransformFromFrameData(InSkeletonData, InFrameData, AppleBoneIndex);

			// find out the offset including translation and rotation in apple arkit component space
			const auto VectorOffset = AppleARKitBoneToComponent.GetTranslation() - AppleARKitBoneToComponent_Ref.GetTranslation();
			const auto RotationOffset = AppleARKitBoneToComponent.GetRotation() * AppleARKitBoneToComponent_Ref.GetRotation().Inverse();
            
			// tranform the ref bone to unreal skeleton component space
			const auto MeshBoneToComponent_Ref = GetRefMeshBoneToComponent(OutPose, MeshBoneIndex);
			// convert it into apple arkit component space
			const auto MeshBoneToComponent_Ref_InAppleSpace = MeshBoneToComponent_Ref * AppleSpaceToMeshSpace.Inverse();
			// apply the offset, so as to get the posed bone transform
			const auto MeshBoneToComponent_Ref_InAppleSpace_WithOffset = FTransform(RotationOffset * MeshBoneToComponent_Ref_InAppleSpace.GetRotation()
                                                                                    , VectorOffset + MeshBoneToComponent_Ref_InAppleSpace.GetTranslation()
                                                                                    , MeshBoneToComponent_Ref_InAppleSpace.GetScale3D());
			// convert back to unreal skeleton component space
			const auto MeshBoneToComponent = MeshBoneToComponent_Ref_InAppleSpace_WithOffset * AppleSpaceToMeshSpace;

			// save all bone-to-component transforms and extract the bone-to-parent transforms later
			MeshBoneToComponents[MeshBoneIndex] = MeshBoneToComponent;
		}
	}

	for (const auto& MeshBoneName : MeshBoneNames)
	{
		const auto MeshBoneIndex = OutPose.GetBoneContainer().GetPoseBoneIndexForBoneName(MeshBoneName);
		if (MeshBoneIndex != INDEX_NONE)
		{
			const auto MeshParentBoneIndex = OutPose.GetBoneContainer().GetParentBoneIndex(MeshBoneIndex);
			FCompactPoseBoneIndex CPIndex = OutPose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
			if (CPIndex != INDEX_NONE)
			{
				OutPose[CPIndex] = MeshParentBoneIndex != INDEX_NONE
									? MeshBoneToComponents[MeshBoneIndex] * MeshBoneToComponents[MeshParentBoneIndex].Inverse()
									: MeshBoneToComponents[MeshBoneIndex];
			}
		}
	}
}
