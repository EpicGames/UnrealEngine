// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"
#include "ARLiveLinkRetargetAsset.h"


class APPLEARKITPOSETRACKINGLIVELINK_API FAppleARKitPoseTrackingLiveLinkModule : public IModuleInterface, public IARLiveLinkRetargetingLogic
{
public:
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
	
	// Implements IARLiveLinkRetargetingLogic
	virtual void BuildPoseFromAnimationData(const UARLiveLinkRetargetAsset& SourceAsset, float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData,
											const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) override;
};

DECLARE_LOG_CATEGORY_EXTERN(LogAppleARKitPoseTracking, Log, All);

DECLARE_STATS_GROUP(TEXT("PoseTracking AR"), STATGROUP_PoseTrackingAR, STATCAT_Advanced);
