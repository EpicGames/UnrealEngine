// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitPoseTrackingLiveLinkRemapAsset.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Features/IModularFeatures.h"
#include "AppleARKitPoseTrackingLiveLink.h"

#include "BonePose.h"

UDEPRECATED_AppleARKitPoseTrackingLiveLinkRemapAsset::UDEPRECATED_AppleARKitPoseTrackingLiveLinkRemapAsset(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	// default to unreal tutorial mesh rigging
	AppleARKitBoneNamesToMeshBoneNames = 
	{
		{TEXT("hips_joint"), 					TEXT("pelvis")},
		{TEXT("spine_2_joint"), 				TEXT("spine_01")},
		{TEXT("spine_4_joint"), 				TEXT("spine_02")},
		{TEXT("spine_7_joint"), 				TEXT("spine_03")},
		{TEXT("left_shoulder_1_joint"), 		TEXT("clavicle_l")},
		{TEXT("left_arm_joint"), 				TEXT("upperarm_l")},
		{TEXT("left_forearm_joint"), 			TEXT("lowerarm_l")},
		{TEXT("left_hand_joint"), 				TEXT("hand_l")},
		{TEXT("left_handIndex_1_joint"),		TEXT("index_01_l")},
		{TEXT("left_handIndex_2_joint"),		TEXT("index_02_l")},
		{TEXT("left_handIndex_3_joint"),		TEXT("index_03_l")},
		{TEXT("left_handMid_1_joint"), 			TEXT("middle_01_l")},
		{TEXT("left_handMid_2_joint"), 			TEXT("middle_02_l")},
		{TEXT("left_handMid_3_joint"), 			TEXT("middle_03_l")},
		{TEXT("left_handPinky_1_joint"),		TEXT("pinky_01_l")},
		{TEXT("left_handPinky_2_joint"),		TEXT("pinky_02_l")},
		{TEXT("left_handPinky_3_joint"),		TEXT("pinky_03_l")},
		{TEXT("left_handRing_1_joint"), 		TEXT("ring_01_l")},
		{TEXT("left_handRing_2_joint"), 		TEXT("ring_02_l")},
		{TEXT("left_handRing_3_joint"), 		TEXT("ring_03_l")},
		{TEXT("left_handThumbStart_joint"),		TEXT("thumb_01_l")},
		{TEXT("left_handThumb_1_joint"),		TEXT("thumb_02_l")},
		{TEXT("left_handThumb_2_joint"), 		TEXT("thumb_03_l")},
		{TEXT("right_shoulder_1_joint"), 		TEXT("clavicle_r")},
		{TEXT("right_arm_joint"), 				TEXT("upperarm_r")},
		{TEXT("right_forearm_joint"), 			TEXT("lowerarm_r")},
		{TEXT("right_hand_joint"), 				TEXT("hand_r")},
		{TEXT("right_handIndex_1_joint"), 		TEXT("index_01_r")},
		{TEXT("right_handIndex_2_joint"), 		TEXT("index_02_r")},
		{TEXT("right_handIndex_3_joint"), 		TEXT("index_03_r")},
		{TEXT("right_handMid_1_joint"), 		TEXT("middle_01_r")},
		{TEXT("right_handMid_2_joint"), 		TEXT("middle_02_r")},
		{TEXT("right_handMid_3_joint"), 		TEXT("middle_03_r")},
		{TEXT("right_handPinky_1_joint"), 		TEXT("pinky_01_r")},
		{TEXT("right_handPinky_2_joint"), 		TEXT("pinky_02_r")},
		{TEXT("right_handPinky_3_joint"), 		TEXT("pinky_03_r")},
		{TEXT("right_handRing_1_joint"), 		TEXT("ring_01_r")},
		{TEXT("right_handRing_2_joint"), 		TEXT("ring_02_r")},
		{TEXT("right_handRing_3_joint"), 		TEXT("ring_03_r")},
		{TEXT("right_handThumbStart_joint"),	TEXT("thumb_01_r")},
		{TEXT("right_handThumb_1_joint"), 		TEXT("thumb_02_r")},
		{TEXT("right_handThumb_2_joint"), 		TEXT("thumb_03_r")},
		{TEXT("neck_1_joint"), 					TEXT("neck_01")},
		{TEXT("head_joint"), 					TEXT("head")},
		{TEXT("left_upLeg_joint"), 				TEXT("thigh_l")},
		{TEXT("left_leg_joint"), 				TEXT("calf_l")},
		{TEXT("left_foot_joint"), 				TEXT("foot_l")},
		{TEXT("left_toes_joint"), 				TEXT("ball_l")},
		{TEXT("right_upLeg_joint"), 			TEXT("thigh_r")},
		{TEXT("right_leg_joint"), 				TEXT("calf_r")},
		{TEXT("right_foot_joint"), 				TEXT("foot_r")},
		{TEXT("right_toes_joint"), 				TEXT("ball_r")}
	};
}

extern FTransform GetRefMeshBoneToComponent(const FCompactPose& OutPose, int BoneIndex);
extern FTransform GetAppleBoneToComponentTransformFromRefSkeleton(const FLiveLinkSkeletonStaticData* InSkeletonData, int BoneIndex);
extern FTransform GetAppleBoneToComponentTransformFromFrameData(const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, int BoneIndex);

void UDEPRECATED_AppleARKitPoseTrackingLiveLinkRemapAsset::BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose)
{
	check(InSkeletonData);
	check(InFrameData);

	// Transform Bone Names
	const TArray<FName>& AppleARKitBoneNames = InSkeletonData->GetBoneNames();

	TArray<FName, TMemStackAllocator<>> MeshBoneNames;
	MeshBoneNames.Reserve(AppleARKitBoneNames.Num());

	TArray<FTransform> MeshBoneToComponents;
	const auto MeshBoneNum = OutPose.GetBoneContainer().GetReferenceSkeleton().GetNum();
	MeshBoneToComponents.Reserve(MeshBoneNum);
	for (auto i = 0; i < MeshBoneNum; ++i)
	{
		MeshBoneToComponents.Emplace(GetRefMeshBoneToComponent(OutPose, i));
	}

	for (const FName& AppleARKitBoneName : AppleARKitBoneNames)
	{
		const FName MeshBoneName = GetRemappedBoneName(AppleARKitBoneName);
		MeshBoneNames.Add(MeshBoneName);
	}

	const FQuat ApplyARKitForwardDir = AppleARKitHumanForward.ToOrientationQuat();
	const FQuat MeshForwardDir = MeshForward.ToOrientationQuat();
	const auto AppleSpaceToMeshSpace = FTransform(ApplyARKitForwardDir).Inverse() * FTransform(MeshForwardDir);

	for (auto AppleBoneIndex = 0; AppleBoneIndex < MeshBoneNames.Num(); ++AppleBoneIndex)
	{
		// for each bone, find out its "offset" from ref pose to the actual pose in apple arkit component space
		// and then apply it to the unreal skeleton
		const auto& MeshBoneName = MeshBoneNames[AppleBoneIndex];
		const auto MeshBoneIndex = OutPose.GetBoneContainer().GetPoseBoneIndexForBoneName(MeshBoneName);
		if (MeshBoneIndex != INDEX_NONE)
		{
            const auto AppleARKitBoneToComponent_Ref = GetAppleBoneToComponentTransformFromRefSkeleton(InSkeletonData, AppleBoneIndex);
			const auto AppleARKitBoneToComponent = GetAppleBoneToComponentTransformFromFrameData(InSkeletonData, InFrameData, AppleBoneIndex);

			// find out the offset including translation and rotation in apple arkit component space
			const auto VectorOffset = AppleARKitBoneToComponent.GetTranslation() - AppleARKitBoneToComponent_Ref.GetTranslation();
			const auto RotationOffset = AppleARKitBoneToComponent.GetRotation() * AppleARKitBoneToComponent_Ref.GetRotation().Inverse();
            
			// tranform the ref bone to unreal skeleton component space
			const auto MeshBoneToComponent_Ref = GetRefMeshBoneToComponent(OutPose, MeshBoneIndex);
			// convert it into apple arkit component space
			const auto MeshBoneToComponent_Ref_InAppleSpace = MeshBoneToComponent_Ref * AppleSpaceToMeshSpace.Inverse();
			// apply the offset, so as to get the posed bone transform
			const auto MeshBoneToComponent_Ref_InAppleSpace_WithOffset = FTransform(RotationOffset * MeshBoneToComponent_Ref_InAppleSpace.GetRotation()
                                                                                    , VectorOffset + MeshBoneToComponent_Ref_InAppleSpace.GetTranslation()
                                                                                    , MeshBoneToComponent_Ref_InAppleSpace.GetScale3D());
			// convert back to unreal skeleton component space
			const auto MeshBoneToComponent = MeshBoneToComponent_Ref_InAppleSpace_WithOffset * AppleSpaceToMeshSpace;

			// save all bone-to-component transforms and extract the bone-to-parent transforms later
			MeshBoneToComponents[MeshBoneIndex] = MeshBoneToComponent;
		}
	}

	for (const auto& MeshBoneName : MeshBoneNames)
	{
		const auto MeshBoneIndex = OutPose.GetBoneContainer().GetPoseBoneIndexForBoneName(MeshBoneName);
		if (MeshBoneIndex != INDEX_NONE)
		{
			const auto MeshParentBoneIndex = OutPose.GetBoneContainer().GetParentBoneIndex(MeshBoneIndex);
			FCompactPoseBoneIndex CPIndex = OutPose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
			if (CPIndex != INDEX_NONE)
			{
				OutPose[CPIndex] = MeshParentBoneIndex != INDEX_NONE
									? MeshBoneToComponents[MeshBoneIndex] * MeshBoneToComponents[MeshParentBoneIndex].Inverse()
									: MeshBoneToComponents[MeshBoneIndex];
			}
		}
	}
}

FName UDEPRECATED_AppleARKitPoseTrackingLiveLinkRemapAsset::GetRemappedBoneName(FName BoneName) const
{
	if (auto Found = AppleARKitBoneNamesToMeshBoneNames.Find(BoneName))
	{
		return *Found;
	}
	return NAME_None;
}
