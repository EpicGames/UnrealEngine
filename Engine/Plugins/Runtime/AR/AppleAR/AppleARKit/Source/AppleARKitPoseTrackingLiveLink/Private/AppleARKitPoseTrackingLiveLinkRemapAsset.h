// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "LiveLinkRetargetAsset.h"

#include "AppleARKitPoseTrackingLiveLinkRemapAsset.generated.h"


UCLASS(Blueprintable, Abstract, Deprecated, meta = (DeprecationMessage="This class is deprecated. Please use \"ARLiveLinkRetargetAsset\" instead."))
class UDEPRECATED_AppleARKitPoseTrackingLiveLinkRemapAsset : public ULiveLinkRetargetAsset
{
	GENERATED_BODY()

public:
	UDEPRECATED_AppleARKitPoseTrackingLiveLinkRemapAsset(const FObjectInitializer& ObjectInitializer);
	virtual void BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) override;

	UPROPERTY(EditAnywhere, Category="LiveLink")
	FVector AppleARKitHumanForward = FVector(-1, 0, 0);

	UPROPERTY(EditAnywhere, Category="LiveLink")
	FVector MeshForward = FVector(0, 1, 0);

	UPROPERTY(EditAnywhere, Category="LiveLink")
	TMap<FName, FName> AppleARKitBoneNamesToMeshBoneNames;

private:
	FName GetRemappedBoneName(FName BoneName) const;
};
