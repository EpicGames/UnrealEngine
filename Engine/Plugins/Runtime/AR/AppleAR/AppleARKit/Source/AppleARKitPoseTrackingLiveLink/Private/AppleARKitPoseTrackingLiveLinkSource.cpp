// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitPoseTrackingLiveLinkSource.h"

#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkAnimationRole.h"

#include "Features/IModularFeatures.h"

TSharedPtr<ILiveLinkSourceARKitPoseTracking> FAppleARKitPoseTrackingLiveLinkSourceFactory::CreateLiveLinkSource()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TSharedPtr <ILiveLinkSourceARKitPoseTracking> Source = MakeShareable(new FAppleARKitPoseTrackingLiveLinkSource());
		LiveLinkClient->AddSource(Source);
		return Source;
	}
	return TSharedPtr<ILiveLinkSourceARKitPoseTracking>();
}

FAppleARKitPoseTrackingLiveLinkSource::FAppleARKitPoseTrackingLiveLinkSource() :
	Client(nullptr)
	, LastFramePublished(0)
	, bNewLiveLinkClient(false)
{
}

void FAppleARKitPoseTrackingLiveLinkSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;

	bNewLiveLinkClient = true;
}

bool FAppleARKitPoseTrackingLiveLinkSource::IsSourceStillValid() const
{
	return Client != nullptr;
}

bool FAppleARKitPoseTrackingLiveLinkSource::RequestSourceShutdown()
{
	Client = nullptr;
	return true;
}

FText FAppleARKitPoseTrackingLiveLinkSource::GetSourceMachineName() const
{
	return FText().FromString(FPlatformProcess::ComputerName());
}

FText FAppleARKitPoseTrackingLiveLinkSource::GetSourceStatus() const
{
	return NSLOCTEXT( "AppleARKitPoseTrackingLiveLink", "AppleARKitPoseTrackingLiveLinkStatus", "Active" );
}

FText FAppleARKitPoseTrackingLiveLinkSource::GetSourceType() const
{
	return NSLOCTEXT( "AppleARKitPoseTrackingLiveLink", "AppleARKitPoseTrackingLiveLinkSourceType", "Apple AR Body Tracking" );
}

void FAppleARKitPoseTrackingLiveLinkSource::UpdateLiveLink(FName SubjectName, const FTimecode& Timecode, uint32 FrameRate, const FARPose3D& PoseData, const FARPose3D& RefPoseData, FName DeviceId)
{
	const FLiveLinkSubjectKey SubjectKey(SourceGuid, SubjectName);

	if (Client)
	{
		// Per ReceiveClient initialization:
		if (bNewLiveLinkClient)
		{
			FLiveLinkStaticDataStruct SkeletalData = SetupLiveLinkStaticData(RefPoseData);
			Client->RemoveSubject_AnyThread(SubjectKey);
			Client->PushSubjectStaticData_AnyThread(SubjectKey, ULiveLinkAnimationRole::StaticClass(), MoveTemp(SkeletalData));
			bNewLiveLinkClient = false;
		}

		// Every frame updates:
		{
			FLiveLinkFrameDataStruct LiveLinkFrameDataStruct(FLiveLinkAnimationFrameData::StaticStruct());
			FLiveLinkAnimationFrameData& LiveLinkAnimationFrameData = *LiveLinkFrameDataStruct.Cast<FLiveLinkAnimationFrameData>();
			LiveLinkAnimationFrameData.WorldTime = FPlatformTime::Seconds();

			//Copy transforms over to transient structure
			// Live link transforms need to be in the hierarchical skeleton, so each in the space of its parent.
			// The hand tracking transforms are in world space.
			TArray<FTransform>& Transforms = LiveLinkAnimationFrameData.Transforms;
			Transforms.Reset();
			for (int32 Index = 0; Index < PoseData.SkeletonDefinition.NumJoints; ++Index)
			{
				int32 ParentIndex = PoseData.SkeletonDefinition.ParentIndices[Index];
				const bool bIsTracked = PoseData.IsJointTracked[Index] && (ParentIndex < 0 || PoseData.IsJointTracked[ParentIndex]);
				const TArray<FTransform>& JointTransforms = bIsTracked ? PoseData.JointTransforms : RefPoseData.JointTransforms;
				const FTransform& BoneTransform = JointTransforms[Index];

				if (ParentIndex < 0)
				{
					// We are at the root, so use it.
					Transforms.Emplace(BoneTransform);
				}
				else
				{
					const FTransform& ParentTransform = JointTransforms[ParentIndex];
					Transforms.Emplace(BoneTransform * ParentTransform.Inverse());
				}
			}

			// Share the data locally with the LiveLink client
			Client->PushSubjectFrameData_AnyThread(SubjectKey, MoveTemp(LiveLinkFrameDataStruct));
		}
	}
}

FLiveLinkStaticDataStruct FAppleARKitPoseTrackingLiveLinkSource::SetupLiveLinkStaticData(const FARPose3D& RefPoseData)
{
	check(IsInGameThread());

	const FARSkeletonDefinition& ARSkeletonDefinition = RefPoseData.SkeletonDefinition;

	FLiveLinkStaticDataStruct LiveLinkSkeletonStaticData;
	LiveLinkSkeletonStaticData.InitializeWith(FLiveLinkSkeletonStaticData::StaticStruct(), nullptr);
	FLiveLinkSkeletonStaticData* SkeletonDataPtr = LiveLinkSkeletonStaticData.Cast<FLiveLinkSkeletonStaticData>();
	check(SkeletonDataPtr);

	SkeletonDataPtr->SetBoneNames(ARSkeletonDefinition.JointNames);
	SkeletonDataPtr->SetBoneParents(ARSkeletonDefinition.ParentIndices);

	RefTransforms.Reserve(RefPoseData.SkeletonDefinition.NumJoints);
	for (int32 Index = 0; Index < RefPoseData.SkeletonDefinition.NumJoints; ++Index)
	{
		const FTransform& BoneTransform = RefPoseData.JointTransforms[Index];
		int32 ParentIndex = RefPoseData.SkeletonDefinition.ParentIndices[Index];
		if (ParentIndex < 0)
		{
			// We are at the root, so use it.
			RefTransforms.Emplace(BoneTransform);
		}
		else
		{
			const FTransform& ParentTransform = RefPoseData.JointTransforms[ParentIndex];
			RefTransforms.Emplace(BoneTransform * ParentTransform.Inverse());
		}
	}

	return LiveLinkSkeletonStaticData;
}