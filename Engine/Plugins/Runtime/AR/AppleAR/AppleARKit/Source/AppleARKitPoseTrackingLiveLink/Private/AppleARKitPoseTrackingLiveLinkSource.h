// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AppleARKitPoseTrackingLiveLinkSourceFactory.h"
#include "ILiveLinkClient.h"
#include "LiveLinkTypes.h"


/** Publishes face blend shapes to LiveLink for use locally */
class FAppleARKitPoseTrackingLiveLinkSource :
	public ILiveLinkSourceARKitPoseTracking
{
public:
	FAppleARKitPoseTrackingLiveLinkSource();
	virtual ~FAppleARKitPoseTrackingLiveLinkSource() {}

private:
	// ILiveLinkSource interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual bool IsSourceStillValid() const override;
	virtual bool RequestSourceShutdown() override;
	virtual FText GetSourceMachineName() const override;
	virtual FText GetSourceStatus() const override;
	virtual FText GetSourceType() const override;
	// End ILiveLinkSource

	void UpdateLiveLink(FName SubjectName, const FTimecode& Timecode, uint32 FrameRate, const FARPose3D& PoseData, const FARPose3D& RefPoseData, FName DeviceId) override;
	const TArray<FTransform>* GetRefPoseTransforms() const override { return &RefTransforms; }
	FLiveLinkStaticDataStruct SetupLiveLinkStaticData(const FARPose3D& RefPoseData);
private:

	/** The local client to push data updates to */
	ILiveLinkClient* Client;

	/** Our identifier in LiveLink */
	FGuid SourceGuid;

	/** The last time we sent the data. Used to not send redundant data */
	uint32 LastFramePublished;

	bool bNewLiveLinkClient;

	TArray<FTransform> RefTransforms;
};
