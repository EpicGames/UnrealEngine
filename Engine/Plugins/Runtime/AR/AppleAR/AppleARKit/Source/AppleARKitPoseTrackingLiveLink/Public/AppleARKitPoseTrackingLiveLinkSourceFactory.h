// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARSystem.h"
#include "ARTrackable.h"
#include "ILiveLinkSource.h"

class UTimecodeProvider;

/** Interface that publishes face ar blend shape information via LiveLink */
class APPLEARKITPOSETRACKINGLIVELINK_API ILiveLinkSourceARKitPoseTracking :
	public ILiveLinkSource
{
public:
	virtual void UpdateLiveLink(FName SubjectName, const FTimecode& Timecode, uint32 FrameRate, const FARPose3D& PoseData, const FARPose3D& RefPoseData, FName DeviceID = NAME_None) = 0;
	virtual const TArray<FTransform>* GetRefPoseTransforms() const = 0;
};

/** Factory that creates and registers the sources with the LiveLink client */
class APPLEARKITPOSETRACKINGLIVELINK_API FAppleARKitPoseTrackingLiveLinkSourceFactory
{
public:
	/** Creates a face mesh source that will autobind to the tracked face mesh */
	static TSharedPtr<ILiveLinkSourceARKitPoseTracking> CreateLiveLinkSource();
};
